// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AnimationEditorWidgets : ModuleRules
{
	public AnimationEditorWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		/** NOTE: THIS MODULE SHOULD NOT EVER DEPEND ON UNREALED. 
		 * Please refer to ToolWidgets.Build.cs for more information.
		 */
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Slate",
				"SlateCore",
				"InputCore",
				"ToolWidgets",
				"AnimationCore",
				"ApplicationCore",
				"PropertyEditor",
				"WidgetRegistration",
				"ToolMenus",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"CoreUObject",
				"UnrealEd",
				"GraphEditor",
			}
		);
	}
}
