// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "SchematicGraphPanel/SchematicGraphStyle.h"
#include "SchematicGraphPanel/SchematicGraphDefines.h"

DEFINE_LOG_CATEGORY(LogSchematicGraph);

class FAnimationEditorWidgetsModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override
	{
		(void)FSchematicGraphStyle::Get();
	}
	
	virtual void ShutdownModule() override
	{
	}
};

FSchematicGraphStyle& FSchematicGraphStyle::Get()
{
	static FSchematicGraphStyle Inst;
	return Inst;
}

IMPLEMENT_MODULE(FAnimationEditorWidgetsModule, AnimationEditorWidgets);
