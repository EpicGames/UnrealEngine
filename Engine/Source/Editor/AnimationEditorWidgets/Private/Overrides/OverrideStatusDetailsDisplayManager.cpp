//  Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/OverrideStatusDetailsDisplayManager.h"
#include "Editor/PropertyEditor/Private/SDetailsView.h"

#define LOCTEXT_NAMESPACE "OverrideStatusDetailsDisplayManager"

FOverrideStatusDetailsDisplayManager::~FOverrideStatusDetailsDisplayManager()
{
}

bool FOverrideStatusDetailsDisplayManager::ShouldShowCategoryMenu()
{
	return false;
}

const FDetailsViewStyleKey& FOverrideStatusDetailsDisplayManager::GetDetailsViewStyleKey() const
{
	return SDetailsView::GetPrimaryDetailsViewStyleKey();
}

bool FOverrideStatusDetailsDisplayManager::CanConstructPropertyUpdatedWidgetBuilder() const
{
	return bIsDisplayingOverrideableObject;
}

TSharedPtr<FPropertyUpdatedWidgetBuilder> FOverrideStatusDetailsDisplayManager::ConstructPropertyUpdatedWidgetBuilder(const FConstructPropertyUpdatedWidgetBuilderArgs& Args)
{
	TSharedPtr<FOverrideStatusDetailsWidgetBuilder> OverridesComboButtonBuilder;
	if (bIsDisplayingOverrideableObject)
	{
		if (Args.InvalidateCachedState.IsBound())
		{
			InvalidateCachedState = Args.InvalidateCachedState;
		}

		OverridesComboButtonBuilder = ConstructOverrideWidgetBuilder( Args );
	}
	return OverridesComboButtonBuilder;
}

void FOverrideStatusDetailsDisplayManager::SetIsDisplayingOverrideableObject(bool InIsDisplayingOverrideableObject)
{
	bIsDisplayingOverrideableObject = InIsDisplayingOverrideableObject;
}

TSharedPtr<FOverrideStatusWidgetMenuBuilder> FOverrideStatusDetailsDisplayManager::GetMenuBuilder(const FOverrideStatusSubject& InSubject) const
{
	const TWeakPtr<FDetailsDisplayManager> WeakManager = ConstCastWeakPtr<FDetailsDisplayManager>(AsWeak());
	return MakeShared<FOverrideStatusWidgetMenuBuilder>(InSubject, StaticCastWeakPtr<FOverrideStatusDetailsDisplayManager>(WeakManager));
}

TSharedPtr<FOverrideStatusDetailsWidgetBuilder> FOverrideStatusDetailsDisplayManager::ConstructOverrideWidgetBuilder( const FConstructPropertyUpdatedWidgetBuilderArgs& Args )
{
	// if we are displaying an overrideable object, create the overrides widget
	if ( bIsDisplayingOverrideableObject )
	{
		TArray<FOverrideStatusObject> Objects;
		if(Args.Objects)
		{
			for(const TWeakObjectPtr<UObject>& ObjectPtr : *Args.Objects)
			{
				if(ObjectPtr.IsValid())
				{
					Objects.Add(ObjectPtr.Get() /* todo Keys */);
				}
			}
		}
		if ( !Objects.IsEmpty() )
		{
			return MakeShared<FOverrideStatusDetailsWidgetBuilder>(SharedThis(this), Objects, Args.PropertyPath, Args.Category);
		}
	}
	// else return nullptr and use the usual reset to default button
	return nullptr;
}

void FOverrideStatusDetailsDisplayManager::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!ObjectBeingModified
		|| !bIsDisplayingOverrideableObject
		|| !PropertyChangedEvent.Property
		|| !InvalidateCachedState.IsBound())
	{
		return;
	}

	InvalidateCachedState.Execute();
}

#undef LOCTEXT_NAMESPACE
