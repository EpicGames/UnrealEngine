// Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/OverrideStatusDetailsObjectFilter.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "PropertyPath.h"

FOverrideStatusDetailsViewObjectFilter::FOverrideStatusDetailsViewObjectFilter()
{
}

void FOverrideStatusDetailsViewObjectFilter::InitializeDisplayManager()
{
	OverrideStatusDisplayManager = MakeShared<FOverrideStatusDetailsDisplayManager>();
	DisplayManager = OverrideStatusDisplayManager;

	CanMergeObjectDelegate.BindStatic(&FOverrideStatusDetailsViewObjectFilter::MergeObjectByClass);
}

TArray<FDetailsViewObjectRoot> FOverrideStatusDetailsViewObjectFilter::FilterObjects(const TArray<UObject*>& SourceObjects)
{
	// this should always be valid while running, ensuring no crash during shutdown
	if (!OverrideStatusDisplayManager.IsValid() || !OnCanCreateWidget().IsBound())
	{
		return {};
	}

	TArray<TArray<UObject*>> ObjectSetList;
	TArray<FDetailsViewObjectRoot> Roots;
	OverrideStatusDisplayManager->SetIsDisplayingOverrideableObject(false);

	for(UObject* SourceObject : SourceObjects)
	{
		if(SourceObject)
		{
			if(OnCanCreateWidget().Execute(FOverrideStatusSubject(SourceObject)))
			{
				if(!ObjectSetList.IsEmpty() && CanMergeObjectDelegate.IsBound())
				{
					bool bMergedObjectIntoSet = false;
					for(TArray<UObject*>& ObjectSet : ObjectSetList)
					{
						for(int32 IndexInSet = 0; IndexInSet < ObjectSet.Num(); IndexInSet++)
						{
							if(CanMergeObjectDelegate.Execute(SourceObject, ObjectSet[IndexInSet]))
							{
								ObjectSet.Add(SourceObject);
								bMergedObjectIntoSet = true;
								break;
							}
						}
					}
					
					if(bMergedObjectIntoSet)
					{
						continue;
					}
				}
				ObjectSetList.Add({SourceObject});
			}
		}
	}

	if(!ObjectSetList.IsEmpty())
	{
		for(const TArray<UObject*>& ObjectSet : ObjectSetList)
		{
			Roots.Emplace(ObjectSet);
		}
		OverrideStatusDisplayManager->SetIsDisplayingOverrideableObject(true);
	}

	return Roots;
}

TSharedPtr<FOverrideStatusWidgetMenuBuilder> FOverrideStatusDetailsViewObjectFilter::GetMenuBuilder(const FOverrideStatusSubject& InSubject) const
{
	return OverrideStatusDisplayManager->GetMenuBuilder(InSubject);
}

FOverrideStatus_CanCreateWidget& FOverrideStatusDetailsViewObjectFilter::OnCanCreateWidget()
{
	return OverrideStatusDisplayManager->OnCanCreateWidget();
}

FOverrideStatus_GetStatus& FOverrideStatusDetailsViewObjectFilter::OnGetStatus()
{
	return OverrideStatusDisplayManager->OnGetStatus();
}

FOverrideStatus_OnWidgetClicked& FOverrideStatusDetailsViewObjectFilter::OnWidgetClicked()
{
	return OverrideStatusDisplayManager->OnWidgetClicked();
}

FOverrideStatus_OnGetMenuContent& FOverrideStatusDetailsViewObjectFilter::OnGetMenuContent()
{
	return OverrideStatusDisplayManager->OnGetMenuContent();
}

FOverrideStatus_AddOverride& FOverrideStatusDetailsViewObjectFilter::OnAddOverride()
{
	return OverrideStatusDisplayManager->OnAddOverride();
}

FOverrideStatus_ClearOverride& FOverrideStatusDetailsViewObjectFilter::OnClearOverride()
{
	return OverrideStatusDisplayManager->OnClearOverride();
}

FOverrideStatus_ResetToDefault& FOverrideStatusDetailsViewObjectFilter::OnResetToDefault()
{
	return OverrideStatusDisplayManager->OnResetToDefault();
}

FOverrideStatus_ValueDiffersFromDefault& FOverrideStatusDetailsViewObjectFilter::OnValueDiffersFromDefault()
{
	return OverrideStatusDisplayManager->OnValueDiffersFromDefault();
}

bool FOverrideStatusDetailsViewObjectFilter::MergeObjectByClass(const UObject* InObjectA, const UObject* InObjectB)
{
	if(InObjectA && InObjectB)
	{
		const UClass* ClassA = InObjectA->GetClass();
		const UClass* ClassB = InObjectB->GetClass();
		if(ClassA && ClassB)
		{
			if(ClassA->IsChildOf(ClassB) || ClassB->IsChildOf(ClassA))
			{
				return true;
			}
		}
	}
	return false;
}
