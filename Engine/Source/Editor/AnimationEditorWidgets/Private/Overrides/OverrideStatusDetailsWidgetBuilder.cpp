//  Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/OverrideStatusDetailsWidgetBuilder.h"

#include "DetailsDisplayManager.h"
#include "Overrides/SOverrideStatusWidget.h"
#include "Overrides/OverrideStatusDetailsDisplayManager.h"
#include "SSimpleComboButton.h"
#include "Styling/SlateBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "DataStorage/Features.h"
#include "Framework/MetaData/DriverMetaData.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "OverrideStatusDetailsWidgetBuilder"

FOverrideStatusDetailsWidgetBuilder::FOverrideStatusDetailsWidgetBuilder(
	const TSharedRef<FOverrideStatusDetailsDisplayManager>& InDetailsDisplayManager,
	const TArray<FOverrideStatusObject>& InObjects,
	const TSharedPtr<const FPropertyPath>& InPropertyPath,
	const FName& InCategory)
	: FPropertyUpdatedWidgetBuilder()
	, DisplayManager(InDetailsDisplayManager)
	, Subject(InObjects, InPropertyPath, InCategory)
{
}

TSharedPtr<SWidget> FOverrideStatusDetailsWidgetBuilder::GenerateWidget()
{
	TSharedPtr<SWidget> StatusWidgets;
	
	if (Subject.IsValid())
	{
		if(OnCanCreateWidget().IsBound())
		{
			if(!OnCanCreateWidget().Execute(Subject))
			{
				return SNullWidget::NullWidget;
			}
		}
		
		TSharedPtr<SBox> Box;
		TSharedPtr<SOverrideStatusWidget> Button;

		auto GetStatusLambda = [this]() -> EOverrideWidgetStatus::Type
		{
			if(OnGetStatus().IsBound())
			{
				if(Subject.IsValid())
				{
					return OnGetStatus().Execute(Subject);
				}
			}
			return EOverrideWidgetStatus::Undetermined;
		};

		SAssignNew(Button, SOverrideStatusWidget)
		.IsHovered(this->IsRowHoveredAttr)
		.Status_Lambda([GetStatusLambda]() -> EOverrideWidgetStatus::Type
		{
			return GetStatusLambda();
		})
		.OnClicked_Lambda([this, GetStatusLambda]() -> FReply
		{
			if(OnWidgetClicked().IsBound())
			{
				if(Subject.IsValid())
				{
					return OnWidgetClicked().Execute(Subject, GetStatusLambda());
				}
			}
			else if(Subject.Num() == 1 && Subject.HasPropertyPath() && OnAddOverride().IsBound() && OnClearOverride().IsBound())
			{
				if(Subject.IsValid())
				{
					if(GetStatusLambda() == EOverrideWidgetStatus::ChangedHere)
					{
						return OnClearOverride().Execute(Subject);
					}
					if(GetStatusLambda() == EOverrideWidgetStatus::None)
					{
						return OnAddOverride().Execute(Subject);
					}
					return FReply::Unhandled();
				}
			}
			return FReply::Unhandled();
		})
		.MenuContent_Lambda([this, GetStatusLambda]() -> TSharedRef<SWidget>
		{
			if(Subject.IsValid())
			{
				if(OnGetMenuContent().IsBound())
				{
					return OnGetMenuContent().Execute(Subject, GetStatusLambda());
				}

				// alternatively rely on the default menu builder
				if(const TSharedPtr<FOverrideStatusWidgetMenuBuilder> MenuBuilder = DisplayManager->GetMenuBuilder(Subject))
				{
					if(const TSharedPtr<SWidget> MenuWidget = MenuBuilder->GenerateWidget())
					{
						return MenuWidget.ToSharedRef();
					}
				}
			}
			return SNullWidget::NullWidget;
		});

		SAssignNew(Box, SBox)
			.WidthOverride(20.0f)
			.HeightOverride(20.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center);
		Box->SetContent(Button.ToSharedRef());

		StatusWidgets = Box;
	}
	
	return StatusWidgets;
}

FOverrideStatusDetailsWidgetBuilder::~FOverrideStatusDetailsWidgetBuilder()
{
}

FOverrideStatus_CanCreateWidget& FOverrideStatusDetailsWidgetBuilder::OnCanCreateWidget()
{
	return DisplayManager->OnCanCreateWidget();
}

FOverrideStatus_GetStatus& FOverrideStatusDetailsWidgetBuilder::OnGetStatus()
{
	return DisplayManager->OnGetStatus();
}

FOverrideStatus_OnWidgetClicked& FOverrideStatusDetailsWidgetBuilder::OnWidgetClicked()
{
	return DisplayManager->OnWidgetClicked();
}

FOverrideStatus_OnGetMenuContent& FOverrideStatusDetailsWidgetBuilder::OnGetMenuContent()
{
	return DisplayManager->OnGetMenuContent();
}

FOverrideStatus_AddOverride& FOverrideStatusDetailsWidgetBuilder::OnAddOverride()
{
	return DisplayManager->OnAddOverride();
}

FOverrideStatus_ClearOverride& FOverrideStatusDetailsWidgetBuilder::OnClearOverride()
{
	return DisplayManager->OnClearOverride();
}

FOverrideStatus_ResetToDefault& FOverrideStatusDetailsWidgetBuilder::OnResetToDefault()
{
	return DisplayManager->OnResetToDefault();
}

FOverrideStatus_ValueDiffersFromDefault& FOverrideStatusDetailsWidgetBuilder::OnValueDiffersFromDefault()
{
	return DisplayManager->OnValueDiffersFromDefault();
}

#undef LOCTEXT_NAMESPACE
