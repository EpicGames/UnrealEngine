//  Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/OverrideStatusSubject.h"

#include "PropertyPath.h"

#define LOCTEXT_NAMESPACE "OverrideStatusSubject"

FOverrideStatusObject::FOverrideStatusObject()
	: Key(NAME_None)
{
}

FOverrideStatusObject::FOverrideStatusObject(const UObject* InObject, const FName& InKey)
	: WeakObjectPtr(TWeakObjectPtr<const UObject>(InObject))
	, Key(InKey)
{
}

FName FOverrideStatusObject::GetFName() const
{
	if(const UObject* Object = GetObject())
	{
		return Object->GetFName();
	}
	return NAME_None;
}

bool FOverrideStatusObject::IsValid() const
{
	return GetObject() != nullptr;
}

const UObject* FOverrideStatusObject::GetObject() const
{
	if(WeakObjectPtr.IsValid())
	{
		return WeakObjectPtr.Get();
	}
	return nullptr;
}

const FName& FOverrideStatusObject::GetKey() const
{
	return Key;
}

bool FOverrideStatusObject::operator==(const FOverrideStatusObject& InOther) const
{
	return (GetObject() == InOther.GetObject()) && (GetKey() == InOther.GetKey());
}

FOverrideStatusSubject::FOverrideStatusSubject(const TArray<FOverrideStatusObject>& InObjects,
                                               const TSharedPtr<const FPropertyPath>& InPropertyPath, const FName& InCategory)
	: Objects(InObjects)
	, PropertyPath(InPropertyPath)
	, Category(InCategory)
{
}

FOverrideStatusSubject::FOverrideStatusSubject(const FOverrideStatusObject& InObject,
	const TSharedPtr<const FPropertyPath>& InPropertyPath, const FName& InCategory)
	: Objects({InObject})
	, PropertyPath(InPropertyPath)
	, Category(InCategory)
{
}

FOverrideStatusSubject::FOverrideStatusSubject(const UObject* InObject, const TSharedPtr<const FPropertyPath>& InPropertyPath,
	const FName& InCategory, const FName& InKey)
	: Objects({FOverrideStatusObject(InObject, InKey)})
	, PropertyPath(InPropertyPath)
	, Category(InCategory)
{
}

bool FOverrideStatusSubject::IsValid() const
{
	for(const FOverrideStatusObject& Object : Objects)
	{
		if(!Object.IsValid())
		{
			return false;
		}
	}
	return !Objects.IsEmpty();
}

int32 FOverrideStatusSubject::Num() const
{
	return Objects.Num();
}

const FOverrideStatusObject& FOverrideStatusSubject::operator[](int32 InIndex) const
{
	return Objects[InIndex];
}

bool FOverrideStatusSubject::HasPropertyPath() const
{
	return PropertyPath.IsValid();
}

const TSharedPtr<const FPropertyPath>& FOverrideStatusSubject::GetPropertyPath() const
{
	return PropertyPath;
}

const FString& FOverrideStatusSubject::GetPropertyPathString(const TCHAR* Separator) const
{
	if(!PropertyPath.IsValid())
	{
		static const FString EmptyString;
		return EmptyString;
	}
	
	if(LastSeparator.IsSet() && LastPropertyPathString.IsSet())
	{
		if(LastSeparator->Equals(Separator))
		{
			return LastPropertyPathString.GetValue();
		}
	}
	
	LastSeparator = FString(Separator);
	LastPropertyPathString = PropertyPath->ToString(Separator);
	return LastPropertyPathString.GetValue();
}

bool FOverrideStatusSubject::HasCategory() const
{
	return !Category.IsNone();
}

const FName& FOverrideStatusSubject::GetCategory() const
{
	return Category;
}

bool FOverrideStatusSubject::Contains(const FOverrideStatusObject& InObject) const
{
	for(const FOverrideStatusObject& Object : Objects)
	{
		if(Object == InObject)
		{
			return true;
		}
	}
	return false;
}

int32 FOverrideStatusSubject::Find(const FOverrideStatusObject& InObject) const
{
	for(int32 Index = 0; Index < Objects.Num(); Index++)
	{
		const FOverrideStatusObject& Object = Objects[Index];
		if(Object == InObject)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE
