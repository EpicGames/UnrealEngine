//  Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/OverrideStatusWidgetMenuBuilder.h"

#include "Overrides/OverrideStatusDetailsDisplayManager.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "UObject/PropertyBagRepository.h"

#define LOCTEXT_NAMESPACE "OverrideStatusWidgetMenuBuilder"

FOverrideStatusWidgetMenuBuilder::FOverrideStatusWidgetMenuBuilder(const FOverrideStatusSubject& InSubject, const TWeakPtr<FOverrideStatusDetailsDisplayManager>& InDisplayManager)
	: FToolElementRegistrationArgs("FOverrideStatusWidgetMenuBuilder")
	, Subject(InSubject)
{
	if(InDisplayManager.IsValid())
	{
		TSharedPtr<FOverrideStatusDetailsDisplayManager> Manager = InDisplayManager.Pin();
		FOverrideStatus_GetStatus GetStatusDelegate = Manager->OnGetStatus();
		StatusAttribute.BindLambda([GetStatusDelegate, this]() -> EOverrideWidgetStatus::Type
		{
			if(GetStatusDelegate.IsBound())
			{
				if(Subject.IsValid())
				{
					return GetStatusDelegate.Execute(Subject);
				}
			}
			return EOverrideWidgetStatus::Undetermined;
		});

		AddOverrideDelegate = Manager->OnAddOverride();
		ClearOverrideDelegate = Manager->OnClearOverride();
		ResetToDefaultDelegate = Manager->OnResetToDefault();
		ValueDiffersFromDefaultDelegate = Manager->OnValueDiffersFromDefault();
	}
	else
	{
		StatusAttribute = EOverrideWidgetStatus::Undetermined;
	}
}

FOverrideStatusWidgetMenuBuilder::~FOverrideStatusWidgetMenuBuilder()
{
}

EOverrideWidgetStatus::Type FOverrideStatusWidgetMenuBuilder::GetStatus() const
{
	return StatusAttribute.Get(EOverrideWidgetStatus::Undetermined);
}

void FOverrideStatusWidgetMenuBuilder::AddOverride()
{
	if(AddOverrideDelegate.IsBound() && Subject.IsValid())
	{
		(void)AddOverrideDelegate.Execute(Subject);
	}
}

bool FOverrideStatusWidgetMenuBuilder::CanAddOverride() const
{
	if(!Subject.IsValid())
	{
		return false;
	}
	static const TSet<EOverrideWidgetStatus::Type> ValidStatusList = {
		EOverrideWidgetStatus::None,
		EOverrideWidgetStatus::Undetermined,
		EOverrideWidgetStatus::ChangedInside,
	};
	return AddOverrideDelegate.IsBound() && ValidStatusList.Contains(GetStatus());
}

FOverrideStatus_AddOverride& FOverrideStatusWidgetMenuBuilder::OnAddOverride()
{
	return AddOverrideDelegate;
}

void FOverrideStatusWidgetMenuBuilder::ClearOverride()
{
	if(ClearOverrideDelegate.IsBound() && Subject.IsValid())
	{
		(void)ClearOverrideDelegate.Execute(Subject);
	}
}

bool FOverrideStatusWidgetMenuBuilder::CanClearOverride() const
{
	if(!Subject.IsValid())
	{
		return false;
	}
	static const TSet<EOverrideWidgetStatus::Type> ValidStatusList = {
		EOverrideWidgetStatus::ChangedHere,
		EOverrideWidgetStatus::ChangedInside,
		EOverrideWidgetStatus::Undetermined,
	};
	return ClearOverrideDelegate.IsBound() && ValidStatusList.Contains(GetStatus());
}

FOverrideStatus_ClearOverride& FOverrideStatusWidgetMenuBuilder::OnClearOverride()
{
	return ClearOverrideDelegate;
}

void FOverrideStatusWidgetMenuBuilder::ResetToDefault()
{
	if(ResetToDefaultDelegate.IsBound() && Subject.IsValid())
	{
		(void)ResetToDefaultDelegate.Execute(Subject);
	}
}

bool FOverrideStatusWidgetMenuBuilder::CanResetToDefault() const
{
	if(!Subject.IsValid())
	{
		return false;
	}
	if(!ValueDiffersFromDefaultDelegate.IsBound())
	{
		return false;
	}
	if(!ValueDiffersFromDefaultDelegate.Execute(Subject))
	{
		return false;
	}
	static const TSet<EOverrideWidgetStatus::Type> ValidStatusList = {
		EOverrideWidgetStatus::ChangedHere,
		EOverrideWidgetStatus::ChangedInside,
		EOverrideWidgetStatus::Undetermined,
	};
	return ResetToDefaultDelegate.IsBound() && ValidStatusList.Contains(GetStatus());
}

FOverrideStatus_ResetToDefault& FOverrideStatusWidgetMenuBuilder::OnResetToDefault()
{
	return ResetToDefaultDelegate;
}

FOverrideStatus_ValueDiffersFromDefault& FOverrideStatusWidgetMenuBuilder::OnValueDiffersFromDefault()
{
	return ValueDiffersFromDefaultDelegate;
}

void FOverrideStatusWidgetMenuBuilder::InitializeMenu()
{
	static const FName CategoryMenuName = "CategoryMenu";
	static const FName PropertyMenuName = "PropertyMenu";
	static const FName Overrides = FName(TEXT("Overrides"));
	const FName ActiveMenu = Subject.HasPropertyPath() ? PropertyMenuName : CategoryMenuName;

	ToolMenu = UToolMenus::Get()->
						RegisterMenu(ActiveMenu,
							NAME_None,
							EMultiBoxType::Menu, false);

	FToolMenuSection& OverrideSection = ToolMenu->AddSection(Overrides);

	TSharedPtr<FOverrideStatusWidgetMenuBuilder> ThisSharedPtr = SharedThis(this);

	OverrideSection.AddMenuEntry(
		TEXT("AddOverride"),
		LOCTEXT("AddOverride", "Add Override"),
		LOCTEXT("AddOverrideTooltip", "Adds an override to this property."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ThisSharedPtr]
			{
				ThisSharedPtr->AddOverride();
			}),
			FCanExecuteAction::CreateLambda([ThisSharedPtr]
			{
				return ThisSharedPtr->CanAddOverride();
			}))
	);

	OverrideSection.AddMenuEntry(
		TEXT("ClearOverride"),
		LOCTEXT("ClearOverride", "Clear Override"),
		LOCTEXT("ClearOverrideTooltip", "Clears the override to this property and resets the value."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ThisSharedPtr]
			{
				ThisSharedPtr->ClearOverride();
			}),
			FCanExecuteAction::CreateLambda([ThisSharedPtr]
			{
				return ThisSharedPtr->CanClearOverride();
			}))
	);

	OverrideSection.AddMenuEntry(
		TEXT("ResetToDefault"),
		LOCTEXT("ResetToDefault", "Reset to default"),
		LOCTEXT("ResetToDefaultTooltip", "Revert the value to its default but keeps the override."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ThisSharedPtr]
			{
				ThisSharedPtr->ResetToDefault();
			}),
			FCanExecuteAction::CreateLambda([ThisSharedPtr]
		{
			return ThisSharedPtr->CanResetToDefault();
		}))
	);

	ToolMenu->bShouldCloseWindowAfterMenuSelection = true;
	ToolMenu->bCloseSelfOnly = true;
}

TSharedPtr<SWidget> FOverrideStatusWidgetMenuBuilder::GenerateWidget()
{
	InitializeMenu();
	
	return ToolMenu.IsValid() ?
		UToolMenus::Get()->GenerateWidget(ToolMenu.Get()) :
		SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
