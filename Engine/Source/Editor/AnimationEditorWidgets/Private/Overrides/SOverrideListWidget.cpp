//  Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/SOverrideListWidget.h"

#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SOverrideListWidget"

SLATE_IMPLEMENT_WIDGET(SOverrideListWidget)

void SOverrideListWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

SOverrideListWidget::SOverrideListWidget()
{
}

SOverrideListWidget::~SOverrideListWidget()
{
};

void SOverrideListWidget::Construct(const FArguments& InArgs)
{
	SubjectsHashAttribute = InArgs._SubjectsHash;
	SubjectsAttribute = InArgs._Subjects;
	GetStatusDelegate = InArgs._OnGetStatus;
	ClearOverrideDelegate = InArgs._OnClearOverride;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(4)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(0)
			[
				SAssignNew(TextBlock, STextBlock)
			]
		]
	];

	SetCanTick(true);
}

void SOverrideListWidget::Tick(
	const FGeometry& AllottedGeometry,
	const double InCurrentTime,
	const float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SOverrideListWidget::Tick);
	
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const uint32 SubjectsHash = SubjectsHashAttribute.Get();
	if(LastHash.Get(UINT32_MAX) != SubjectsHash)
	{
		LastHash = SubjectsHash;
		
		const TArray<FOverrideStatusSubject>& Subjects = SubjectsAttribute.Get();

		FText Content;

		for(const FOverrideStatusSubject& Subject : Subjects)
		{
			if(Subject.IsValid())
			{
				Content = FText::Format(
					LOCTEXT("SubjectFormat", "{0}\n{1} {2}"),
					Content,
					FText::FromName(Subject[0].GetFName()),
					FText::FromString(Subject.GetPropertyPathString())
				);
			}
		}

		TextBlock->SetText(Content);
	}
}

#undef LOCTEXT_NAMESPACE
