//  Copyright Epic Games, Inc. All Rights Reserved.

#include "Overrides/SOverrideStatusWidget.h"

#include "Components/HorizontalBox.h"
#include "SSimpleComboButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SOverrideStatusWidget"

SLATE_IMPLEMENT_WIDGET(SOverrideStatusWidget)

void SOverrideStatusWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

SOverrideStatusWidget::SOverrideStatusWidget()
	: WidgetStyle(nullptr)
{
}

SOverrideStatusWidget::~SOverrideStatusWidget()
{
};

void SOverrideStatusWidget::Construct(const FArguments& InArgs)
{
	StatusAttribute = InArgs._Status;
	OnGetMenuContent = InArgs._MenuContent;
	OnClicked = InArgs._OnClicked;
	bAlwaysUpdateOnTick = false;
	bTickIsPending = false;

	if(InArgs._IsHovered.IsBound())
	{
		SetHover(InArgs._IsHovered);
	}
	
	if(InArgs._DefaultStyle.IsSet())
	{
		DefaultWidgetStyle = InArgs._DefaultStyle.GetValue();
	}
	else
	{
		if(const FOverrideStatusWidgetStyle* StyleFromStatus = GetStyleFromStatus(EOverrideWidgetStatus::Undetermined))
		{
			DefaultWidgetStyle = *StyleFromStatus;
		}
	}
	
	WidgetStyle = nullptr;
	OnGetStyle = InArgs._Style;
	OnGetTooltip = InArgs._Tooltip;

	if(!OnGetStyle.IsBound())
	{
		OnGetStyle.BindStatic(&SOverrideStatusWidget::GetStyleFromStatus_Fallback);
		bAlwaysUpdateOnTick = StatusAttribute.IsBound();
	}

	ChildSlot
	[
		SAssignNew(Image, SImage)
	];
}

FReply SOverrideStatusWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

void SOverrideStatusWidget::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	if(!GetHoveredAttribute().IsBound())
	{
		SetHover(true);
	}
	QueueRepaint();
}

void SOverrideStatusWidget::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	if(!GetHoveredAttribute().IsBound())
	{
		SetHover(false);
	}
	QueueRepaint();
}

void SOverrideStatusWidget::QueueRepaint()
{
	Invalidate(EInvalidateWidgetReason::Paint);
	SetCanTick(true);
	bTickIsPending = true;
}

FReply SOverrideStatusWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
	return HandleClick();
}

FReply SOverrideStatusWidget::HandleClick()
{
	// Create the overrides menu at the cursor location
	if(OnClicked.IsBound())
	{
		const FReply ReplyFromDelegate = OnClicked.Execute();
		if(ReplyFromDelegate.IsEventHandled())
		{
			return ReplyFromDelegate;
		}
	}
	if (OnGetMenuContent.IsBound())
	{
		const TSharedRef<SWidget> Content = OnGetMenuContent.Execute();
		if(Content != SNullWidget::NullWidget)
		{
			FSlateApplication::Get().PushMenu(
				SharedThis(this),
				FWidgetPath(),
				Content,
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup));
			return FReply::Handled();
		}
	}
	
	return FReply::Unhandled();
}

EOverrideWidgetStatus::Type SOverrideStatusWidget::GetStatus() const
{
	return StatusAttribute.Get(EOverrideWidgetStatus::Undetermined);
}

void SOverrideStatusWidget::SetStatus(EOverrideWidgetStatus::Type InStatus)
{
	StatusAttribute = InStatus;
	QueueRepaint();
}

const FOverrideStatusWidgetStyle* SOverrideStatusWidget::GetStyleFromStatus_Fallback(EOverrideWidgetStatus::Type InStatus)
{
	return GetStyleFromStatus(InStatus);
}

void SOverrideStatusWidget::Tick(
	const FGeometry& AllottedGeometry,
	const double InCurrentTime,
	const float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SOverrideStatusWidget::Tick);
	
	if(!bAlwaysUpdateOnTick)
	{
		// Only need to tick on first run or if the data changes which trigger a repaint
		// Paint will tick us.
		SetCanTick(false);
	}
	else
	{
		bTickIsPending = true;
	}

	if(bTickIsPending)
	{
		// Figure out what type of Icon we need
		const FOverrideStatusWidgetStyle* Style = nullptr;
		if (OnGetStyle.IsBound())
		{
			Style = OnGetStyle.Execute(GetStatus());
		}

		const FOverrideStatusWidgetStyle* NewWidgetStyle = Style ? Style : &DefaultWidgetStyle;
		if(NewWidgetStyle != WidgetStyle)
		{
			WidgetStyle = NewWidgetStyle;
			
			Image->SetDesiredSizeOverride(WidgetStyle->IconSize);
			Image->SetToolTipText(WidgetStyle->Tooltip);

			bLastHovered.Reset();
		}

		const bool bIsHovered = IsHovered();
		if (bIsHovered != bLastHovered.Get(!bIsHovered))
		{
			bLastHovered = bIsHovered;
			if(bIsHovered)
			{
				Image->SetImage(WidgetStyle->HoveredIcon);
				Image->SetColorAndOpacity(WidgetStyle->HoveredColor);
			}
			else
			{
				Image->SetImage(WidgetStyle->Icon);
				Image->SetColorAndOpacity(WidgetStyle->Color);
			}
		}

		bTickIsPending = false;
	}
	
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

const FOverrideStatusWidgetStyle* SOverrideStatusWidget::GetStyleFromStatus(EOverrideWidgetStatus::Type InStatus)
{
	static const TMap<EOverrideWidgetStatus::Type, FOverrideStatusWidgetStyle> StyleMap =
		{
			{ // No override
				EOverrideWidgetStatus::None,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideNone")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideNone.Hovered")),
					.Tooltip = LOCTEXT("NoOverridePropertyToolTip", "No override."),
					.IconSize = FVector2D(16, 16),
				}
			},
			{ // Undetermined
				EOverrideWidgetStatus::Undetermined,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideUndetermined")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideUndetermined")),
					.Tooltip = LOCTEXT("UndeterminedPropertyToolTip", "State has not yet been determined"),
					.IconSize = FVector2D(16, 16),
				},
			},
			{ // uninitialized
				EOverrideWidgetStatus::Uninitialized,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideAlert")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideAlert.Hovered")),
					.Tooltip = LOCTEXT("NonConcreteUninitialized", "This property needs a value.\nYou won't be able to test or publish until you set one."),
					.Color = FStyleColors::Error,
					.IconSize = FVector2D(16, 16),
				}
			},
			{ // Inherited
				EOverrideWidgetStatus::Inherited,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideInherited")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideInherited.Hovered")),
					.Tooltip = LOCTEXT("InheritedPropertyToolTip", "This property's parent component has been overridden."),
					.HoveredColor = FColor::White,
					.IconSize = FVector2D(16, 16),
				}
			},
			{ // Here
				EOverrideWidgetStatus::ChangedHere,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideHere")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideHere.Hovered")),
					.Tooltip = LOCTEXT("HerePropertyToolTip", "This property has been overridden."),
					.HoveredColor = FColor::White,
					.IconSize = FVector2D(16, 16),
				}
			},
			{ // Inside
				EOverrideWidgetStatus::ChangedInside,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideInside")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideInside.Hovered")),
					.Tooltip = LOCTEXT("InsidePropertyToolTip", "At least one of this property's values has been overridden."),
					.HoveredColor = FColor::White,
					.IconSize = FVector2D(16, 16),
				}
			},
			{ // Outside
				EOverrideWidgetStatus::ChangedOutside,
				{
					.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideInherited")),
					.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideInherited.Hovered")),
					.Tooltip = LOCTEXT("OverrideInheritedToolTip", "A parent of this property has been overridden."),
					.HoveredColor = FColor::White,
					.IconSize = FVector2D(16, 16),
				}
			},
			{ // Mixed
            	EOverrideWidgetStatus::Mixed,
            	{
            		.Icon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideMixed")),
            		.HoveredIcon = FAppStyle::GetBrush(TEXT("DetailsView.OverrideMixed.Hovered")),
            		.Tooltip = LOCTEXT("OverrideMixedToolTip", "The selected elements have mixed override states on this property."),
					.HoveredColor = FColor::White,
            		.IconSize = FVector2D(16, 16),
            	}
            }
		};

	return StyleMap.Find(InStatus);
}

#undef LOCTEXT_NAMESPACE
