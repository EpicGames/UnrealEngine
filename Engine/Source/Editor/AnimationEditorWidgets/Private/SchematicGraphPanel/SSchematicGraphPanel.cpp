// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SchematicGraphPanel/SSchematicGraphPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Engine/Font.h"
#include "Engine/Engine.h"
#include <SchematicGraphPanel/SchematicGraphStyle.h>
#include "Fonts/FontMeasure.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "SSchematicGraphPanel"

TSharedRef<FSchematicGraphNodeDragDropOp> FSchematicGraphNodeDragDropOp::New(TArray<SSchematicGraphNode*> InSchematicGraphNodes, const TArray<FGuid>& InElements)
{
	TSharedRef<FSchematicGraphNodeDragDropOp> Operation = MakeShared<FSchematicGraphNodeDragDropOp>();
	Operation->SchematicGraphNodes = InSchematicGraphNodes;
	Operation->Elements = InElements;
	Operation->Construct();
	return Operation;
}

FSchematicGraphNodeDragDropOp::~FSchematicGraphNodeDragDropOp()
{
}

TSharedPtr<SWidget> FSchematicGraphNodeDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
			.Text(FText::FromString(GetJoinedDecoratorLabels()))
		];
}

const TArray<const FSchematicGraphNode*> FSchematicGraphNodeDragDropOp::GetNodes() const
{
	TArray<const FSchematicGraphNode*> Result;
	for(SSchematicGraphNode* Node : SchematicGraphNodes)
	{
		Result.Add(Node->GetNodeData());
	}
	return Result;
}

FString FSchematicGraphNodeDragDropOp::GetJoinedDecoratorLabels() const
{
	TArray<FString> DecoratorLabels;
	for(const SSchematicGraphNode* GraphNode : SchematicGraphNodes)
	{
		if(const FSchematicGraphNode* Node = GraphNode->GetNodeData())
		{
			DecoratorLabels.Add(Node->GetDragDropDecoratorLabel());
		}
	}
	return FString::Join(DecoratorLabels, TEXT(","));
}

SSchematicGraphNode::FArguments::FArguments()
: _NodeData(nullptr)
, _EnableAutoScale(false)
, _BrushGetter(nullptr)
{
	static const FSchematicGraphNode EmptyNodeData;
	_NodeData = &EmptyNodeData;

	_BrushGetter = [](const FGuid&,int32) -> const FSlateBrush*
	{
		static const FSlateBrush* WhiteTexture = FAppStyle::GetBrush("WhiteTexture");
		return WhiteTexture;
	};
}

void SSchematicGraphNode::Construct(const FArguments& InArgs)
{
	if(InArgs._NodeData)
	{
		NodeData = const_cast<FSchematicGraphNode*>(InArgs._NodeData)->AsShared();
	}
	OnClickedDelegate = InArgs._OnClicked;
	OnBeginDragDelegate = InArgs._OnBeginDrag;
	OnEndDragDelegate = InArgs._OnEndDrag;
	OnDropDelegate = InArgs._OnDrop;

	Position = InArgs._Position;
	Size = InArgs._Size;
	Scale = InArgs._Scale;
	EnableAutoScale = InArgs._EnableAutoScale;
	LayerColors = InArgs._LayerColors;
	BrushGetter = InArgs._BrushGetter;
	OriginalSize = Size->Get();

	if(InArgs._ToolTipText.IsBound() || InArgs._ToolTipText.IsSet())
	{
		SetToolTipText(InArgs._ToolTipText);
	}

	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SSchematicGraphNode::GetNodeVisibility));

	if(const FSchematicGraphGroupNode* GroupNode = Cast<FSchematicGraphGroupNode>(GetNodeData()))
	{
		ExpansionCircleFactor = FFloatAttribute::Create(GroupNode->GetAnimationSettings(), 0.f);
	}
}

FVector2D SSchematicGraphNode::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return OriginalSize*LayoutScaleMultiplier;
}

int32 SSchematicGraphNode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const int32 NewLayerId = SNodePanel::SNode::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const FVector2d CurSize = Size->Get() * Scale->Get();
	const FVector2d SizeOffset = (CurSize-OriginalSize)*-0.5;
	static const UFont* Font = GEngine->GetSmallFont();
	static const FSlateFontInfo SmallFontStyle = Font->GetLegacySlateFontInfo();
	const bool bIsFadedOut = IsFadedOut();
	const float FadedOutFactor = bIsFadedOut ? 0.5f : 1.f;

	const int32 FadedOutGroupLayerId = NewLayerId;
	const int32 FadedOutNodeLayerId = NewLayerId + 100;
	const int32 FocusedGroupLayerId = NewLayerId + 200;
	const int32 FocusedNodeLayerId = NewLayerId + 300;
	int32 NodeLayerId = bIsFadedOut ? FadedOutNodeLayerId : FocusedNodeLayerId;
	int32 GroupLayerId = bIsFadedOut ? FadedOutGroupLayerId : FocusedGroupLayerId;

	if(const FSchematicGraphGroupNode* GroupNode = Cast<FSchematicGraphGroupNode>(GetNodeData()))
	{
		if(GroupNode->GetExpansionState() > SMALL_NUMBER)
		{
			ExpansionCircleFactor->Set(GroupNode == GroupNode->GetGraph()->GetLastExpandedNode() ? 1.f : 0.f);
		}
		else
		{
			ExpansionCircleFactor->Set(0.f);
		}

		if(ExpansionCircleFactor.IsValid() && ExpansionCircleFactor->Get() > SMALL_NUMBER)
		{
			check(ExpansionCircleFactor.IsValid());
			
			ExpansionCircleFactor->Set(1.f);
			
			static const FSlateBrush* GroupBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Group");
			const float Radius = GroupNode->GetExpansionState() * (GroupNode->GetExpansionRadius() + FMath::Max(OriginalSize.X, OriginalSize.Y) * 0.525f);
			const FLinearColor Color = GroupNode->GetExpansionColor() * ExpansionCircleFactor->Get();

			const FVector2d CircleSize = FVector2d::One() * Radius * 2.f;
			const FVector2d CircleOffset = (CircleSize-OriginalSize)*-0.5;

			NodeLayerId++;

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				NodeLayerId,
				AllottedGeometry.ToPaintGeometry(CircleSize, FSlateLayoutTransform(CircleOffset)),
				GroupBrush,
				ESlateDrawEffect::None,
				Color * FadedOutFactor
			);
		}
	}

	if(BrushGetter)
	{
		for(int32 LayerIndex = 0; LayerIndex < LayerColors.Num(); LayerIndex++)
		{
			NodeLayerId++;

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				NodeLayerId,
				AllottedGeometry.ToPaintGeometry(CurSize, FSlateLayoutTransform(SizeOffset)),
				BrushGetter(GetGuid(), LayerIndex),
				ESlateDrawEffect::None,
				LayerColors[LayerIndex].IsValid() ? LayerColors[LayerIndex]->Get() : FLinearColor::White
			);
		}
	}

	if (NodeData && SchematicGraphPanel)
	{
		if(TSharedPtr<FSchematicGraphModel> SchematicGraph = SchematicGraphPanel->GetSchematicGraphModel().Pin())
		{
			const float NodeRadius = FMath::Min(CurSize.X, CurSize.Y) * 0.5;
		
			const TArray<TSharedPtr<FSchematicGraphTag>>& Tags = NodeData->GetTags();
			for(const TSharedPtr<FSchematicGraphTag>& TagPtr : Tags)
			{
				const FSchematicGraphTag* CurrentTag = TagPtr.Get();
				const ESchematicGraphVisibility::Type TagVisibility = SchematicGraph->GetVisibilityForTag(CurrentTag);
				if(TagVisibility == ESchematicGraphVisibility::Hidden)
				{
					continue;
				}

				FVector2d TagSize = FVector2d::One() * NodeRadius * 1.0f;
				FVector2d LabelSize = FVector2d::ZeroVector;
				const FText& Label = SchematicGraph->GetLabelForTag(CurrentTag);
				const FString LabelString = Label.ToString();

				if(!LabelString.IsEmpty())
				{
					const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
					LabelSize = FontMeasureService->Measure(LabelString, SmallFontStyle);
					TagSize.X = FMath::Max(TagSize.X, LabelSize.X + 4);
					TagSize.Y = FMath::Max(TagSize.Y, LabelSize.Y + 4);
				}

				const FVector2d TagCenter = SizeOffset + CurSize * 0.5 + FVector2d(0, NodeRadius).GetRotated(CurrentTag->GetPlacementAngle());
				const FVector2d TagOffset = TagCenter - TagSize * 0.5;

				if(const FSlateBrush* BackgroundBrush = CurrentTag->GetBackgroundBrush())
				{
					NodeLayerId++;

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						NodeLayerId,
						AllottedGeometry.ToPaintGeometry(TagSize, FSlateLayoutTransform(TagOffset)),
						BackgroundBrush,
						ESlateDrawEffect::None,
						SchematicGraph->GetBackgroundColorForTag(CurrentTag) * FadedOutFactor
					);
				}

				if(const FSlateBrush* ForegroundBrush = SchematicGraph->GetForegroundBrushForTag(CurrentTag))
				{
					NodeLayerId++;

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						NodeLayerId,
						AllottedGeometry.ToPaintGeometry(TagSize, FSlateLayoutTransform(TagOffset)),
						ForegroundBrush,
						ESlateDrawEffect::None,
						SchematicGraph->GetForegroundColorForTag(CurrentTag) * FadedOutFactor
					);
				}

				if(!LabelSize.IsNearlyZero())
				{
					NodeLayerId++;
					const FVector2d LabelOffset = TagCenter - LabelSize * 0.5;

					FSlateDrawElement::MakeText(
						OutDrawElements,
						NodeLayerId,
						AllottedGeometry.ToPaintGeometry(LabelSize, FSlateLayoutTransform(LabelOffset)),
						LabelString,
						SmallFontStyle,
						ESlateDrawEffect::None,
						SchematicGraph->GetLabelColorForTag(CurrentTag) * FadedOutFactor
					);
				}
			}
		}
	}

	FText NodeLabel;
	bool bDrawLabel = true;
	if(bDrawLabel)
	{
		const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos() - Args.GetWindowToDesktopTransform();
		if((AllottedGeometry.GetAbsolutePositionAtCoordinates(FVector2d(0.5f, 0.5f)) - MouseCursorLocation).Length() > AllottedGeometry.GetAbsoluteSize().GetMax() * 0.5f + 8.f)
		{
			bDrawLabel = false;
		}
	}
	if(bDrawLabel)
	{
		if(const FSchematicGraphGroupNode* GroupNode = NodeData->GetGroupNode())
		{
			if(GroupNode->IsExpanding() || GroupNode->IsCollapsing())
			{
				bDrawLabel = false;
			}
		}
	}
	if(bDrawLabel)
	{
		NodeLabel = NodeData->GetLabel();
		if(NodeLabel.IsEmpty())
		{
			bDrawLabel = false;
		}
	}
	if(bDrawLabel)
	{
		const FString NodeLabelString = NodeLabel.ToString();
		const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		FVector2d NodeLabelSize = FontMeasureService->Measure(NodeLabelString, SmallFontStyle);

		const FVector2d NodeBottomCenter = AllottedGeometry.GetLocalSize() * FVector2d(0.5f, 1.f) + FVector2d(0.f, 8.f);
		FVector2d NodeLabelOffset = NodeBottomCenter - NodeLabelSize * FVector2d(0.5, 0);
		if(SchematicGraphPanel)
		{
			NodeLabelOffset += SchematicGraphPanel->GetNodeLabelOffset();
		}

		static const FVector2d LabelBackgroundPadding = FVector2d(2.f);
		const FVector2d LabelBackgroundSize = NodeLabelSize + LabelBackgroundPadding * 2.f;
		const FVector2d LabelBackgroundOffset = NodeLabelOffset - LabelBackgroundPadding;
		static const FSlateBrush* LabelBackgroundBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Label.Background");

		static const FColor LabelBackgroundColorHex = FColor::FromHex(TEXT("#0F0F0F"));
		static const FLinearColor LabelBackgroundColor = FLinearColor(LabelBackgroundColorHex) * FLinearColor(1.f, 1.f, 1.f, 0.7f); 
		
		NodeLayerId++;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			NodeLayerId,
			AllottedGeometry.ToPaintGeometry(LabelBackgroundSize, FSlateLayoutTransform(LabelBackgroundOffset)),
			LabelBackgroundBrush,
			ESlateDrawEffect::None,
			LabelBackgroundColor * FadedOutFactor
		);

		NodeLayerId++;
		FSlateDrawElement::MakeText(
			OutDrawElements,
			NodeLayerId,
			AllottedGeometry.ToPaintGeometry(NodeLabelSize, FSlateLayoutTransform(NodeLabelOffset)),
			NodeLabelString,
			SmallFontStyle,
			ESlateDrawEffect::None,
			FLinearColor::White * FadedOutFactor
		);

		if(SchematicGraphPanel)
		{
			SchematicGraphPanel->IncrementNodeLabelOffset(FVector2d(0, LabelBackgroundSize.Y));
		}
	}
	
	return NodeLayerId;
}

void SSchematicGraphNode::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(!IsInteractive() || IsFadedOut())
	{
		return;
	}
	SNode::OnMouseEnter(MyGeometry, MouseEvent);

	if(NodeData)
	{
		NodeData->OnMouseEnter();
	}
}

void SSchematicGraphNode::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if(!IsInteractive() || IsFadedOut())
	{
		return;
	}
	SNode::OnMouseLeave(MouseEvent);

	if(NodeData)
	{
		NodeData->OnMouseLeave();
	}
}

FReply SSchematicGraphNode::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(!IsInteractive() || IsFadedOut())
	{
		return FReply::Unhandled();
	}
	SNode::OnDragOver(MyGeometry, DragDropEvent);

	if(NodeData)
	{
		NodeData->OnDragOver();
	}
	return FReply::Handled();
	
}

void SSchematicGraphNode::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if(!IsInteractive() || IsFadedOut())
	{
		return;
	}
	SNode::OnDragLeave(DragDropEvent);

	if(NodeData)
	{
		NodeData->OnDragLeave();
	}
}

FReply SSchematicGraphNode::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(!IsInteractive() || IsFadedOut())
	{
		return FReply::Unhandled();
	}

	// Avoid dropping onto itself
	const TSharedPtr<FSchematicGraphNodeDragDropOp> SchematicDragDropOp = DragDropEvent.GetOperationAs<FSchematicGraphNodeDragDropOp>();
	if (SchematicDragDropOp)
	{
		if (!SchematicDragDropOp->GetElements().IsEmpty())
		{
			if (SchematicDragDropOp->GetElements().ContainsByPredicate([this](const FGuid& Guid)
			{
				return Guid == NodeData->GetGuid();
			}))
			{
				return FReply::Unhandled();
			}
		}
	}
	
	SNode::OnDrop(MyGeometry, DragDropEvent);
	OnDropDelegate.ExecuteIfBound(this, DragDropEvent);
	OnEndDragDelegate.ExecuteIfBound(this, DragDropEvent.GetOperation());
	return FReply::Handled();
}

FReply SSchematicGraphNode::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FGuid Guid = GetGuid();
	if(SchematicGraphPanel)
	{
		const FReply ReplyFromPanel = SchematicGraphPanel->HandleNodeDragDetected(Guid, MyGeometry, MouseEvent);
		if(ReplyFromPanel.IsEventHandled())
		{
			return ReplyFromPanel;
		}
	}
	
	TArray<FGuid> DraggedElements = {Guid};
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && DraggedElements.Num() > 0)
	{
		if(SchematicGraphPanel)
		{
			if(TSharedPtr<FSchematicGraphModel> SchematicGraph = SchematicGraphPanel->GetSchematicGraphModel().Pin())
			{
				if(SchematicGraph->IsDragSupportedForNode(GetGuid()))
				{
					bIsBeingDragged = true;

					const FVector2f AbsoluteMousePosition = MouseEvent.GetScreenSpacePosition();
					const FVector2d LocalMousePosition =  (AbsoluteMousePosition - MyGeometry.GetAbsolutePosition()) / MyGeometry.GetAccumulatedLayoutTransform().GetScale();
					OffsetDuringDrag = -LocalMousePosition;
			
					const TSharedRef<FSchematicGraphNodeDragDropOp> DragDropOp = FSchematicGraphNodeDragDropOp::New({this}, MoveTemp(DraggedElements));
					OnBeginDragDelegate.ExecuteIfBound(this, DragDropOp.ToSharedPtr());
					return FReply::Handled().BeginDragDrop(DragDropOp);
				}
			}
		}
	}
	
	return FReply::Unhandled();
}

EVisibility SSchematicGraphNode::GetNodeVisibility() const
{
	if(IsBeingDragged())
	{
		return EVisibility::HitTestInvisible;
	}

	if(SchematicGraphPanel)
	{
		if(TSharedPtr<FSchematicGraphModel> SchematicGraph = SchematicGraphPanel->GetSchematicGraphModel().Pin())
		{
			ESchematicGraphVisibility::Type Vis = SchematicGraph->GetVisibilityForNode(GetGuid());
			if(Vis == ESchematicGraphVisibility::Hidden)
			{
				return EVisibility::Hidden;
			}
		}
	}

	return EVisibility::Visible;
}

FReply SSchematicGraphNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SNode::OnMouseButtonDown(MyGeometry, MouseEvent);
	if (MouseEvent.GetPressedButtons().Contains(EKeys::LeftMouseButton))
	{
		OnClickedDelegate.ExecuteIfBound(this, MouseEvent);
	}

	if (MouseEvent.GetPressedButtons().Contains(EKeys::RightMouseButton))
	{
		if(SchematicGraphPanel && GetNodeData())
		{
			if (TSharedPtr<FSchematicGraphModel> Graph = SchematicGraphPanel->GetSchematicGraphModel().Pin())
			{
				FMenuBuilder MenuBuilder(true, nullptr);
				if(Graph->GetContextMenuForNode(GetNodeData(), MenuBuilder))
				{
					TSharedPtr<SWidget> MenuContent = MenuBuilder.MakeWidget();
					if ( MenuContent.IsValid() )
					{
						FVector2D SummonLocation = MouseEvent.GetScreenSpacePosition();
						FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
						FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
					}
				}
			}
		}
	}

	const bool bFadedOut = IsFadedOut();
	
	if(NodeData)
	{
		FReply NodeReply = NodeData->OnClicked(MouseEvent);
		if(NodeReply.IsEventHandled())
		{
			// only allow drag on non-faded nodes
			if(!bFadedOut)
			{
				return NodeReply.DetectDrag(SharedThis(this), EKeys::LeftMouseButton); 
			}
			return NodeReply;
		}
	}
	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FVector2f SSchematicGraphNode::GetPosition2f() const
{
	return UE::Slate::CastToVector2f(Position->Get() - (OriginalSize*0.5));
}

void SSchematicGraphNode::EnablePositionAnimation(bool bEnabled)
{
	Position->EnableInterpolation(bEnabled);
}

const FGuid SSchematicGraphNode::GetGuid() const
{
	return GetNodeData()->GetGuid();
}

bool SSchematicGraphNode::IsInteractive() const
{
	if(NodeData)
	{
		return NodeData->IsInteractive();
	}
	return true;
}

const bool SSchematicGraphNode::IsFadedOut() const
{
	if(NodeData)
	{
		if(const FSchematicGraphModel* Graph = NodeData->GetGraph())
		{
			return Graph->GetVisibilityForNode(GetNodeData()) == ESchematicGraphVisibility::FadedOut;
		}
	}
	return false;
}

void SSchematicGraphPanel::SetSchematicGraphModel(TWeakPtr<FSchematicGraphModel> InGraphData)
{
	if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		GraphData->OnNodeAdded().RemoveAll(this);
		GraphData->OnNodeRemoved().RemoveAll(this);
		GraphData->OnLinkAdded().RemoveAll(this);
		GraphData->OnLinkRemoved().RemoveAll(this);
		GraphData->OnGraphReset().RemoveAll(this);
	}
	
	GraphDataWeak = InGraphData;

	if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		GraphData->OnNodeAdded().AddSP(this, &SSchematicGraphPanel::AddNode);
		GraphData->OnNodeRemoved().AddSP(this, &SSchematicGraphPanel::RemoveNode);
		GraphData->OnLinkAdded().AddSP(this, &SSchematicGraphPanel::AddLink);
		GraphData->OnLinkRemoved().AddSP(this, &SSchematicGraphPanel::RemoveLink);
		GraphData->OnGraphReset().AddSP(this, &SSchematicGraphPanel::RebuildPanel);
	}
}

void SSchematicGraphPanel::Construct(const FArguments& InArgs)
{
	GraphDataWeak = InArgs._GraphDataModel;
	bIsOverlay = InArgs._IsOverlay;
	PaddingLeft = InArgs._PaddingLeft;
	PaddingRight = InArgs._PaddingRight;
	PaddingTop = InArgs._PaddingTop;
	PaddingBottom = InArgs._PaddingBottom;
	PaddingInterNode = InArgs._PaddingInterNode;
	OnNodeClickedDelegate = InArgs._OnNodeClicked;
	OnBeginDragDelegate = InArgs._OnBeginDrag;
	OnEndDragDelegate = InArgs._OnEndDrag;
	OnEnterDragDelegate = InArgs._OnEnterDrag;
	OnLeaveDragDelegate = InArgs._OnLeaveDrag;
	OnDropDelegate = InArgs._OnDrop;
	
	SNodePanel::Construct();

	if(InArgs._Visibility.IsBound() || InArgs._Visibility.IsSet())
	{
		SetVisibility(InArgs._Visibility);
	}
	else
	{
		SetVisibility(bIsOverlay ? EVisibility::SelfHitTestInvisible : EVisibility::Visible);
	}
	
	if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		GraphData->OnNodeAdded().AddSP(this, &SSchematicGraphPanel::AddNode);
		GraphData->OnNodeRemoved().AddSP(this, &SSchematicGraphPanel::RemoveNode);
		GraphData->OnLinkAdded().AddSP(this, &SSchematicGraphPanel::AddLink);
		GraphData->OnLinkRemoved().AddSP(this, &SSchematicGraphPanel::RemoveLink);
		GraphData->OnGraphReset().AddSP(this, &SSchematicGraphPanel::RebuildPanel);
		GraphData->ApplyToPanel(this);
	};
}

void SSchematicGraphPanel::RebuildPanel()
{
	RemoveAllNodes();

	if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		for (const TSharedPtr<FSchematicGraphNode>& Node : GraphData->GetNodes())
		{
			AddNode(Node.Get());
		}
		for (const TSharedPtr<FSchematicGraphLink>& Link : GraphData->GetLinks())
		{
			AddLink(Link.Get());
		}
	}
}

void SSchematicGraphPanel::AddNode(const FSchematicGraphNode* InNodeToAdd)
{
	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();
	if (!GraphData)
	{
		return;
	}
	if(NodeByGuid.Contains(InNodeToAdd->GetGuid()))
	{
		return;
	}
	
	static const TEasingAttributeInterpolator<FVector2d>::FSettings Vector2DInterpolationSettings(EEasingInterpolatorType::CubicEaseOut, 0.2f);
	static const TEasingAttributeInterpolator<float>::FSettings FloatInterpolationSettings(EEasingInterpolatorType::CubicEaseOut, 0.2f);
	static const TEasingAttributeInterpolator<FLinearColor>::FSettings ColorInterpolationSettings(EEasingInterpolatorType::CubicEaseOut, 0.2f);

	const FGuid Guid = InNodeToAdd->GetGuid();
	const auto Position = FVector2dAttribute::CreateWithGetter(Vector2DInterpolationSettings, FVector2dAttribute::FGetter::CreateSP(this, &SSchematicGraphPanel::GetPositionForNode, Guid));
	const auto Size = FVector2dAttribute::CreateWithGetter(Vector2DInterpolationSettings, FVector2dAttribute::FGetter::CreateSP(GraphData.Get(), &FSchematicGraphModel::GetSizeForNode, InNodeToAdd));
	const auto Scale = FFloatAttribute::CreateWithGetter(FloatInterpolationSettings, FFloatAttribute::FGetter::CreateSP(this, &SSchematicGraphPanel::GetScaleForNode, Guid), 0.f);

	const int32 NumLayers = GraphData ? GraphData->GetNumLayersForNode(InNodeToAdd) : InNodeToAdd->GetNumLayers();
	TArray<TSharedPtr<FLinearColorAttribute>> Colors;
	for(int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
	{
		Colors.Add(FLinearColorAttribute::CreateWithGetter(ColorInterpolationSettings, FLinearColorAttribute::FGetter::CreateSP(this, &SSchematicGraphPanel::GetColorForNode, Guid, LayerIndex)));
	}

	const TFunction<const FSlateBrush*(const FGuid&, int32)> BrushGetter = [GraphDataWeak = this->GraphDataWeak](const FGuid& InGuid, int32 InLayerIndex) -> const FSlateBrush*
	{
		if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
		{
			return GraphData->GetBrushForNode(InGuid, InLayerIndex);
		}
		return nullptr;
	};

	const TSharedRef<SSchematicGraphNode> NewNode = SNew(SSchematicGraphNode)
														.Position(Position)
														.Size(Size)
														.Scale(Scale)
														.LayerColors(Colors)
														.EnableAutoScale(this, &SSchematicGraphPanel::IsAutoScaleEnabledForNode, InNodeToAdd->GetGuid())
														.BrushGetter(BrushGetter)
														.ToolTipText(this, &SSchematicGraphPanel::GetToolTipForNode, InNodeToAdd->GetGuid())
														.OnClicked(this, &SSchematicGraphPanel::OnNodeClicked)
														.OnBeginDrag(this, &SSchematicGraphPanel::OnBeginDragEvent)
														.OnEndDrag(this, &SSchematicGraphPanel::OnEndDragEvent)
														.OnDrop(this, &SSchematicGraphPanel::OnDropEvent)
														.NodeData(InNodeToAdd);
	SNodePanel::AddGraphNode(NewNode);
	NewNode->SchematicGraphPanel = this;
	NodeByGuid.Add(Guid, NewNode.ToSharedPtr());
}

void SSchematicGraphPanel::RemoveNode(const FSchematicGraphNode* InNodeToRemove)
{
	const FGuid GuidToRemove = InNodeToRemove->GetGuid();
	NodeByGuid.Remove(GuidToRemove);
	
	for (int32 Iter = Children.Num()-1; Iter >= 0; --Iter)
	{
		TSharedRef<SSchematicGraphNode> Widget = GetChild(Iter);
		if (Widget->GetGuid() == GuidToRemove)
		{
			Children.RemoveAt(Iter);
			break;
		}
	}
	for (int32 Iter = 0; Iter != VisibleChildren.Num(); ++Iter)
	{
		TSharedRef<SSchematicGraphNode> Widget = StaticCastSharedRef<SSchematicGraphNode>(VisibleChildren[Iter]);
		if (Widget->GetGuid() == GuidToRemove)
		{
			VisibleChildren.RemoveAt(Iter);
			break;
		}
	}
}

const SSchematicGraphNode* SSchematicGraphPanel::FindNode(const FGuid& InGuid) const
{
	if(const TSharedPtr<SSchematicGraphNode>* FoundNodePtr = NodeByGuid.Find(InGuid))
	{
		return FoundNodePtr->Get();
	}
	return nullptr;
}

SSchematicGraphNode* SSchematicGraphPanel::FindNode(const FGuid& InGuid)
{
	const SSchematicGraphPanel* ConstThis = this;
	return const_cast<SSchematicGraphNode*>(ConstThis->FindNode(InGuid));
}

void SSchematicGraphPanel::AddLink(const FSchematicGraphLink* InLinkToAdd)
 {
	if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		static const TEasingAttributeInterpolator<float>::FSettings FloatInterpolationSettings(EEasingInterpolatorType::CubicEaseOut, 0.1f);
		static const TEasingAttributeInterpolator<float>::FSettings SlowFloatInterpolationSettings(EEasingInterpolatorType::CubicEaseOut, 0.2f);
		static const TEasingAttributeInterpolator<FLinearColor>::FSettings ColorInterpolationSettings(EEasingInterpolatorType::CubicEaseOut, 0.2f);

		const FGuid Guid = InLinkToAdd->GetGuid();
		const auto Minimum = FFloatAttribute::CreateWithGetter(SlowFloatInterpolationSettings, FFloatAttribute::FGetter::CreateSP(GraphData.Get(), &FSchematicGraphModel::GetMinimumForLink, InLinkToAdd), 0.5f);
		const auto Maximum = FFloatAttribute::CreateWithGetter(SlowFloatInterpolationSettings, FFloatAttribute::FGetter::CreateSP(GraphData.Get(), &FSchematicGraphModel::GetMaximumForLink, InLinkToAdd), 0.5f);
		const auto Color = FLinearColorAttribute::CreateWithGetter(ColorInterpolationSettings, FLinearColorAttribute::FGetter::CreateSP(GraphData.Get(), &FSchematicGraphModel::GetColorForLink, InLinkToAdd));
		const auto Thickness = FFloatAttribute::CreateWithGetter(FloatInterpolationSettings, FFloatAttribute::FGetter::CreateSP(GraphData.Get(), &FSchematicGraphModel::GetThicknessForLink, InLinkToAdd), 0);

		FSchematicLinkWidgetInfo Info;
		Info.Minimum = Minimum;
		Info.Maximum = Maximum;
		Info.Color = Color;
		Info.Thickness = Thickness;

		LinkByGuid.Add(Guid, MakeShareable(new FSchematicLinkWidgetInfo(Info)));
	}
 }

 void SSchematicGraphPanel::RemoveLink(const FSchematicGraphLink* InLinkToRemove)
 {
	const FGuid GuidToRemove = InLinkToRemove->GetGuid();
	LinkByGuid.Remove(GuidToRemove);
 }

 const SSchematicGraphPanel::FSchematicLinkWidgetInfo* SSchematicGraphPanel::FindLink(const FGuid& InGuid) const
 {
	if(const TSharedPtr<FSchematicLinkWidgetInfo>* FoundLinkPtr = LinkByGuid.Find(InGuid))
	{
		return FoundLinkPtr->Get();
	}
	return nullptr;
 }

SSchematicGraphPanel::FSchematicLinkWidgetInfo* SSchematicGraphPanel::FindLink(const FGuid& InGuid)
{
	const SSchematicGraphPanel* ConstThis = this;
	return const_cast<FSchematicLinkWidgetInfo*>(ConstThis->FindLink(InGuid));
}

void SSchematicGraphPanel::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	SNodePanel::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

int32 SSchematicGraphPanel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const int32 BackgroundLayer = LayerId + 1;
	const int32 OutlineLayer = BackgroundLayer + 1;
	const int32 LinkLayerId = OutlineLayer + 2;
	const int32 NodeLayerId = LinkLayerId + 1;
    int32 MaxLayerId = NodeLayerId;
    	
	if (!bIsOverlay)
	{
		const FSlateBrush* DefaultBackground = FAppStyle::GetBrush(TEXT("Graph.Panel.SolidBackground"));
		PaintBackgroundAsLines(DefaultBackground, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
		MaxLayerId++;
	}
	
	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();
	if (!GraphData)
	{
		return MaxLayerId;
	}

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildNodes(AllottedGeometry, ArrangedChildren);

	NodeCenterByGuid.Reset();
	NodeCenterByGuid.Reserve(ArrangedChildren.Num());
	NodeCenterByIndex.Reset();
	NodeCenterByIndex.Reserve(ArrangedChildren.Num());
	NodeVisibilityByIndex.Reset();
	NodeVisibilityByIndex.Reserve(ArrangedChildren.Num());
	NodeVisibilityByGuid.Reset();
	NodeVisibilityByGuid.Reserve(ArrangedChildren.Num());
	
	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		const TSharedRef<SSchematicGraphNode> ChildNode = StaticCastSharedRef<SSchematicGraphNode>(CurWidget.Widget);
		
		const FVector2d NodeCenter = CurWidget.Geometry.GetLocalPositionAtCoordinates({0.5, 0.5});
		NodeCenterByIndex.Add(NodeCenter);
		NodeCenterByGuid.Add(ChildNode->GetGuid(), NodeCenter);

		const FSchematicGraphNode* NodeData = ChildNode->GetNodeData();
		
		const int32 IndexInPerNodeCache = GuidToNodeCache.FindChecked(NodeData->GetGuid());
		ESchematicGraphVisibility::Type NodeVisibility = PerNodeCaches[IndexInPerNodeCache].Visibility;
		
		if(CurWidget.Geometry.GetLocalSize().IsNearlyZero() ||
			!FSlateRect::DoRectanglesIntersect( CurWidget.Geometry.GetLayoutBoundingRect(), MyCullingRect ))
		{
			NodeVisibility = ESchematicGraphVisibility::Hidden;
		}

		NodeVisibilityByIndex.Add(NodeVisibility);
		NodeVisibilityByGuid.Add(NodeData->GetGuid(), NodeVisibility);
	}

	// update the node visibility and centers based on the group relationships
	for (const TSharedPtr<FSchematicGraphNode>& Node : GraphData->GetNodes())
	{
		if(Node->IsRootNode())
		{
			continue;
			
		}

		// if the node is already visible
		if(const ESchematicGraphVisibility::Type* ChildVisibility = NodeVisibilityByGuid.Find(Node->GetGuid()))
		{
			if(*ChildVisibility != ESchematicGraphVisibility::Hidden)
			{
				continue;
			}
		}

		const FGuid RootGuid = Node->GetRootNodeGuid();
		if(const ESchematicGraphVisibility::Type* RootNodeVisibility = NodeVisibilityByGuid.Find(RootGuid))
		{
			// update the child node's visibility + center
			if(*RootNodeVisibility != ESchematicGraphVisibility::Hidden)
			{
				NodeVisibilityByGuid.FindOrAdd(Node->GetGuid()) = *RootNodeVisibility;
				const FVector2d& RootNodeCenter = NodeCenterByGuid.FindChecked(RootGuid);
				NodeCenterByGuid.FindOrAdd(Node->GetGuid()) = RootNodeCenter;
			}
		}
	}

	// draw all of the links
	/*
	for(const TPair<FGuid, TSharedPtr<FSchematicLinkWidgetInfo>>& Pair : LinkByGuid)
	{
		if(GraphData->GetVisibilityForLink(Pair.Key) == ESchematicGraphVisibility::Hidden)
		{
			continue;
		}

		const FSchematicGraphLink* Link = GraphData->FindLink(Pair.Key);
		if(Link == nullptr)
		{
			continue;
		}

		// the node may have been culled
		const ESchematicGraphVisibility::Type* IsSourceNodeVisible = NodeVisibilityByGuid.Find(Link->GetSourceNodeGuid()); 
		const ESchematicGraphVisibility::Type* IsTargetNodeVisible = NodeVisibilityByGuid.Find(Link->GetTargetNodeGuid());
		if((IsSourceNodeVisible == nullptr) || (IsTargetNodeVisible == nullptr))
		{
			continue;
		}
		if((*IsSourceNodeVisible == ESchematicGraphVisibility::Hidden) || (*IsTargetNodeVisible == ESchematicGraphVisibility::Hidden))
		{
			continue;
		}

		const bool bFadedOut = (*IsSourceNodeVisible == ESchematicGraphVisibility::FadedOut) || (*IsTargetNodeVisible == ESchematicGraphVisibility::FadedOut);
		
		const SSchematicGraphNode* SourceNode = FindNode(Link->GetSourceNodeGuid()); 
		const SSchematicGraphNode* TargetNode = FindNode(Link->GetTargetNodeGuid());
		if(SourceNode == nullptr || TargetNode == nullptr || SourceNode == TargetNode)
		{
			continue;
		}

		const FLinearColor Color = Pair.Value->Color->Get() * (bFadedOut ? 0.5f : 1.f);
		const float Thickness = Pair.Value->Thickness->Get();
		const FSlateBrush* Brush = GraphData->GetBrushForLink(Pair.Key);
		const FVector2d& SourcePosition = NodeCenterByGuid.FindChecked(SourceNode->GetGuid()) + GraphData->GetSourceNodeOffsetForLink(Link);
		const FVector2d& TargetPosition = NodeCenterByGuid.FindChecked(TargetNode->GetGuid()) + GraphData->GetTargetNodeOffsetForLink(Link);

		if(SourcePosition.IsNearlyZero() || TargetPosition.IsNearlyZero())
		{
			continue;
		}

		const FVector2d Diff = TargetPosition - SourcePosition;
		const float DiffLength = Diff.Size();
		if(DiffLength < SMALL_NUMBER)
		{
			continue;
		}
		
		const float Minimum = Pair.Value->Minimum->Get();
		const float Maximum = Pair.Value->Maximum->Get();

		const float SourceMinimumDistance = GetMinimumLinkDistanceForNode(SourceNode->GetGuid());
		const float TargetMinimumDistance = GetMinimumLinkDistanceForNode(TargetNode->GetGuid());

		if(DiffLength <= (SourceMinimumDistance + TargetMinimumDistance))
		{
			continue;
		}
		
		const FVector2d DiffNormal = Diff / DiffLength;
		const FVector2d MinimumPosition = SourcePosition + DiffNormal * SourceMinimumDistance;
		const FVector2d MaximumPosition = TargetPosition - DiffNormal * TargetMinimumDistance;

		const TArray<FVector2D> LinePoints = {
			FMath::Lerp<FVector2d>(MinimumPosition, MaximumPosition, FMath::Clamp(Minimum, 0, 1)),
			FMath::Lerp<FVector2d>(MinimumPosition, MaximumPosition, FMath::Clamp(Maximum, 0, 1))
		};

		if(Brush == nullptr)
		{
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LinkLayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				Color,
				true,
				Thickness
			);
		}
		else
		{
			const FVector2d Center = (LinePoints[0] + LinePoints[1]) * 0.5f;
			const float Distance = (LinePoints[0] - LinePoints[1]).Size();
			static constexpr float DefaultDistance = 128.f;
			const float AdjustedThickness = Thickness * FMath::Min(1, Distance / DefaultDistance);
			const FVector2d LineSize = {AdjustedThickness, Distance };
			const FVector2d SizeOffset = LineSize * 0.5f;
			const float Angle = -FMath::Atan2(-Diff.X, -Diff.Y);

			FSlateDrawElement::MakeRotatedBox(
				OutDrawElements,
				LinkLayerId,
				AllottedGeometry.ToPaintGeometry(LineSize, FSlateLayoutTransform(Center - SizeOffset)),
				Brush,
				ESlateDrawEffect::None,
				Angle,
				SizeOffset, // rotation point
				FSlateDrawElement::ERotationSpace::RelativeToElement,
				Color
				);
		}
	}
	*/

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	// Draw the child nodes
	{
		for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
		{
			if(NodeVisibilityByIndex[ChildIndex] == ESchematicGraphVisibility::Hidden)
			{
				continue;
			}
			
			FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
			TSharedRef<SSchematicGraphNode> ChildNode = StaticCastSharedRef<SSchematicGraphNode>(CurWidget.Widget);
			
			// Examine node to see what layers we should be drawing in
			const int32 ChildLayerId = NodeLayerId;
			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, MyCullingRect, OutDrawElements, ChildLayerId, InWidgetStyle, true );
			MaxLayerId = FMath::Max( MaxLayerId, CurWidgetsMaxLayerId + 1 );
		}
	}

	// Draw the software cursor
	++MaxLayerId;
	PaintSoftwareCursor(AllottedGeometry, MyCullingRect, OutDrawElements, MaxLayerId);

	return MaxLayerId;
}

void SSchematicGraphPanel::RemoveAllNodes()
{
	NodeByGuid.Reset();
	SNodePanel::RemoveAllNodes();
}

FReply SSchematicGraphPanel::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// disable mouse wheel for now
	return FReply::Unhandled();
}

TSharedRef<SSchematicGraphNode> SSchematicGraphPanel::GetChild(int32 ChildIndex) const
{
	return StaticCastSharedRef<SSchematicGraphNode>(Children[ChildIndex]);
}

TStatId SSchematicGraphPanel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(SSchematicGraphPanel, STATGROUP_Tickables);
}

static bool IsInMinimizedOrBackgrounded(const TSharedPtr<SWidget>& Widget)
{
	if (!Widget.IsValid())
	{
		return true;
	}

	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(Widget.ToSharedRef());
	if (!Window.IsValid()
		|| Window->IsWindowMinimized())
	{
		return true;
	}

	return false;
}

void SSchematicGraphPanel::Tick(float DeltaTime)
{
	if (IsInMinimizedOrBackgrounded(AsShared()))
	{
		return;
	}
	
	DPIScale.Reset();
	NodeLabelOffset = FVector2d::ZeroVector;
	
	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();
	if (GraphData)
	{
		GraphData->Tick(DeltaTime);
	}
	
	FSlateApplication& Application = FSlateApplication::Get();
	if (Application.IsDragDropping())
	{
		TSharedPtr<FDragDropOperation> DragDropOp = Application.GetDragDroppingContent();
		if (DragDropOp.IsValid())
		{
			const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();
			if(GetPaintSpaceGeometry().GetRenderBoundingRect().ContainsPoint(MouseCursorLocation))
			{
				// if we haven't seen this operation yet we need to let our model know
				if(!DragDropOpFromOutside.IsValid())
				{
					DragDropOpFromOutside = DragDropOp;
					OnEnterDragEvent(DragDropOpFromOutside);
				}
			}

			bIsDragDropping = true;
		}
	}
	else
	{
		bIsDragDropping = false;
		if(DragDropOpFromOutside)
		{
			OnLeaveDragEvent(DragDropOpFromOutside);
		}
		DragDropOpFromOutside.Reset();
	}

	for (int32 i=0; i<Children.Num(); ++i)
	{
		TSharedRef<SSchematicGraphNode> Widget = GetChild(i);

		if(!bIsDragDropping && Widget->IsBeingDragged())
		{
			Widget->bIsBeingDragged = false;
		}

		// update the animation state of the node
		Widget->EnablePositionAnimation(GraphData ? GraphData->GetPositionAnimationEnabledForNode(Widget->GetGuid()) : false);
		if(Widget->GetVisibility() != EVisibility::Visible)
		{
			continue;
		}

		if (Widget->IsBeingDragged())
		{
			if (bIsDragDropping)
			{
				const FVector2f AbsoluteMousePosition = FSlateApplication::Get().GetCursorPos();
				const FGeometry& Geometry = GetTickSpaceGeometry();

				const FVector2d LocalMousePosition =  (AbsoluteMousePosition - Geometry.GetAbsolutePosition()) / Geometry.GetAccumulatedLayoutTransform().GetScale();
				const FVector2d HalfOriginalSize = Widget->GetOriginalSize() * 0.5;
				Widget->PositionDuringDrag = LocalMousePosition + HalfOriginalSize;
			}
			else if (DeltaTime > 0.f)
			{
				Widget->PositionDuringDrag.Reset();
				Widget->OffsetDuringDrag.Reset();
				Widget->bIsBeingDragged = false;
			}
		}
	}

	UpdatePerNodeCaches(true);
	UpdateAutoGroupingForNodes();
	UpdatePerNodeCaches(false);
	UpdateAutoScalingForNodes();
}

void SSchematicGraphPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (IsInMinimizedOrBackgrounded(AsShared()))
	{
		return;
	}
	
	SNodePanel::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	Tick(InDeltaTime);
}

void SSchematicGraphPanel::ToggleVisibility()
{
	const EVisibility PreviousVisibility = GetVisibility();
	SetVisibility(
		PreviousVisibility == EVisibility::Hidden ?
		EVisibility::SelfHitTestInvisible :
		EVisibility::Hidden);
}

void SSchematicGraphPanel::OnNodeClicked(SSchematicGraphNode* Node, const FPointerEvent& MouseEvent)
{
	OnNodeClickedDelegate.ExecuteIfBound(this, Node, MouseEvent);
}

void SSchematicGraphPanel::OnBeginDragEvent(SSchematicGraphNode* Node, const TSharedPtr<FDragDropOperation>& InDragDropOp)
{
	DropTarget.Reset();
	OnBeginDragDelegate.ExecuteIfBound(this, Node, InDragDropOp);
}

void SSchematicGraphPanel::OnEndDragEvent(SSchematicGraphNode* Node, const TSharedPtr<FDragDropOperation>& InDragDropOp)
{
	if(!DropTarget.IsSet())
	{
		OnCancelDragEvent(Node, InDragDropOp);
	}
	OnEndDragDelegate.ExecuteIfBound(this, Node, InDragDropOp);
	DropTarget.Reset();
}

void SSchematicGraphPanel::OnEnterDragEvent(const TSharedPtr<FDragDropOperation>& InDragDropEvent)
{
	OnEnterDragDelegate.ExecuteIfBound(this, InDragDropEvent);
}

void SSchematicGraphPanel::OnLeaveDragEvent(const TSharedPtr<FDragDropOperation>& InDragDropEvent)
{
	OnLeaveDragDelegate.ExecuteIfBound(this, InDragDropEvent);
}

void SSchematicGraphPanel::OnCancelDragEvent(SSchematicGraphNode* Node, const TSharedPtr<FDragDropOperation>& InDragDropEvent)
{
	OnCancelDragDelegate.ExecuteIfBound(this, Node, InDragDropEvent);
	DropTarget.Reset();
}

void SSchematicGraphPanel::OnDropEvent(SSchematicGraphNode* Node, const FDragDropEvent& InDragDropEvent)
{
	DropTarget = Node ? Node->GetGuid() : FGuid(); 
	OnDropDelegate.ExecuteIfBound(this, Node, InDragDropEvent);
}

FReply SSchematicGraphPanel::HandleNodeDragDetected(FGuid Guid, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		FGuid ForwardedGuid = Guid;
		if(GraphData->GetForwardedNodeForDrag(ForwardedGuid))
		{
			SSchematicGraphNode* ForwardedNode = const_cast<SSchematicGraphNode*>(FindNode(ForwardedGuid));
			return ForwardedNode->OnDragDetected(MyGeometry, MouseEvent);
		}
	}
	return FReply::Unhandled();
}

FVector2d SSchematicGraphPanel::GetPositionForNode(FGuid InNodeGuid) const
{
	if(const SSchematicGraphNode* NodeWidget = FindNode(InNodeGuid))
	{
		if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
		{
			if(NodeWidget->PositionDuringDrag.IsSet())
			{
				return NodeWidget->PositionDuringDrag.GetValue() + NodeWidget->OffsetDuringDrag.Get(FVector2d::ZeroVector);
			}
		
			if(const FSchematicGraphNode* Node = GraphData->FindNode(InNodeGuid))
			{
				FVector2d Position = GraphData->GetPositionOffsetForNode(Node);
				Position += GraphData->GetPositionForNode(Node);

				AdjustPositionWithDPIScale(Position);
				return Position;
			}
		}
	}
	return FVector2d::ZeroVector;
}

FLinearColor SSchematicGraphPanel::GetColorForNode(FGuid InNodeGuid, int32 InLayerIndex) const
{
	if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		return GraphData->GetColorForNode(InNodeGuid, InLayerIndex);
	}
	return FLinearColor::White;
}

FText SSchematicGraphPanel::GetToolTipForNode(FGuid InNodeGuid) const
{
	if (TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		return GraphData->GetToolTipForNode(InNodeGuid);
	}
	return FText();
}

float SSchematicGraphPanel::GetScaleForNode(FGuid InNodeGuid) const
{
	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();

	// check if the node may be auto scaled
	if(const TSharedPtr<SSchematicGraphNode>* NodePtr = NodeByGuid.Find(InNodeGuid))
	{
		if(NodePtr->Get()->AutoScale.IsSet())
		{
			float ScaleOffset = 1.f;
			if(GraphData)
			{
				ScaleOffset = GraphData->GetScaleOffsetForNode(NodePtr->Get()->GetNodeData());
			}
			return NodePtr->Get()->AutoScale.GetValue() * ScaleOffset;
		}
	}
	
	if(GraphData)
	{
		return GraphData->GetScaleForNode(InNodeGuid);
	}
	return 1.f;
}

void SSchematicGraphPanel::AdjustPositionWithDPIScale(FVector2d& InOutPosition, bool bInverse) const
{
	if(!DPIScale.IsSet())
	{
		const float WidgetX = CachedGeometry.GetAbsolutePosition().X;
		const float WidgetY = CachedGeometry.GetAbsolutePosition().Y;
		DPIScale = 1.f / FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WidgetX, WidgetY);
	}
	if (bInverse)
	{
		InOutPosition /= DPIScale.GetValue();
	}
	else
	{
		InOutPosition *= DPIScale.GetValue();
	}
}

bool SSchematicGraphPanel::IsAutoGroupingEnabled() const
{
	if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		return GraphData->IsAutoGroupingEnabled();
	}
	return false;
}

float SSchematicGraphPanel::GetAutoGroupingDistance() const
{
	if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		return GraphData->GetAutoGroupingDistance();
	}
	return 0.f;
}

bool SSchematicGraphPanel::IsAutoScaleEnabledForNode(FGuid InNodeGuid) const
{
	if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		return GraphData->IsAutoScaleEnabledForNode(InNodeGuid);
	}
	return false;
}

 float SSchematicGraphPanel::GetMinimumLinkDistanceForNode(FGuid InLinkGuid, bool bIncludeScale) const
 {
	if(TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin())
	{
		const float MinimumDistance = GraphData->GetMinimumLinkDistanceForNode(InLinkGuid);
		if(bIncludeScale)
		{
			const float Scale = GetScaleForNode(InLinkGuid);
			return MinimumDistance * Scale;
		}
		return MinimumDistance;
	}
	return 0.f;
 }

void SSchematicGraphPanel::IncrementNodeLabelOffset(const FVector2d& InOffset)
{
	NodeLabelOffset += InOffset + FVector2d(0,3);
}

void SSchematicGraphPanel::UpdatePerNodeCaches(bool bRemoveNodesFromAutoGroups)
{
	PerNodeCaches.Reset();
	GuidToNodeCache.Reset();

	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();
	if(GraphData == nullptr)
	{
		return;
	}

	PerNodeCaches.Reserve(Children.Num());
	GuidToNodeCache.Reserve(Children.Num());

	for (int32 i=0; i<Children.Num(); ++i)
	{
		TSharedRef<SSchematicGraphNode> Widget = GetChild(i);

		FPerNodeCache Cache;
		if(FSchematicGraphNode* Node = Widget->GetNodeData())
		{
			if(bRemoveNodesFromAutoGroups)
			{
				if(Cast<FSchematicGraphAutoGroupNode>(Node->GetParentNode()))
				{
					GraphData->RemoveFromParentNode(Node, false);
				}
			}
			Cache.Guid = Node->GetGuid();
			Cache.Label = Node->GetLabel();
			Cache.bHasParent = Node->HasParentNode();
			Cache.Visibility = GraphData->GetVisibilityForNode(Node);
			Cache.bIsAutoScaling = !Cache.bHasParent && !Widget->bIsBeingDragged && Widget->EnableAutoScale.Get();
			Cache.Position = GetPositionForNode(Node->GetGuid());
			const FVector2d NodeSize = GraphData->GetSizeForNode(Node->GetGuid());
			// note: this is not necessarily the best way to determine the radius of a node
			Cache.Radius = FMath::Min(NodeSize.X, NodeSize.Y) * 0.5;
		}

		const int32 Index = PerNodeCaches.Add(Cache);
		GuidToNodeCache.Add(Cache.Guid, Index);
	}
}

void SSchematicGraphPanel::UpdateAutoGroupingForNodes()
{
	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();
	if(GraphData == nullptr)
	{
		return;
	}

	if(!IsAutoGroupingEnabled())
	{
		return;
	}

	TMap<uint32, FSchematicGraphGroupNode*> GroupNodeByHash;
	for(const TPair<uint32, FGuid>& Pair : GroupNodeGuidByHash)
	{
		if(FSchematicGraphGroupNode* GroupNode = Cast<FSchematicGraphGroupNode>(GraphData->FindNode(Pair.Value)))
		{
			GroupNodeByHash.Add(Pair.Key, GroupNode);
		}
	}

	// update the group nodes as needed
	const float AutoGroupingDistance = GetAutoGroupingDistance();
	TMap<uint32, TArray<FGuid>> NodeGuidsPerHash;
	TMap<uint32, FVector2d> NodePositionPerHash;
	for (int32 i=0; i<Children.Num(); ++i)
	{
		if(PerNodeCaches[i].bHasParent)
		{
			continue;
		}
		if(PerNodeCaches[i].Visibility == ESchematicGraphVisibility::Hidden)
		{
			continue;
		}

		TSharedRef<SSchematicGraphNode> Widget = GetChild(i);
		const FSchematicGraphNode* Node = Widget->GetNodeData();
		if(Node->IsA<FSchematicGraphAutoGroupNode>())
		{
			continue;
		}

		const FVector2d FloatingPointPosition = PerNodeCaches[i].Position;
		const TTuple<int32,int32> IntegerPosition = {
			FMath::RoundToInt(FloatingPointPosition.X / AutoGroupingDistance),
			FMath::RoundToInt(FloatingPointPosition.Y / AutoGroupingDistance)
		};
		
		const uint32 PositionHash = HashCombine(IntegerPosition.Get<0>(), IntegerPosition.Get<1>());
		NodeGuidsPerHash.FindOrAdd(PositionHash).Add(Node->GetGuid());

		if(!NodePositionPerHash.Contains(PositionHash))
		{
			NodePositionPerHash.Add(PositionHash, FloatingPointPosition);
		}
	}

	// remove all groups which has 0 or 1 element.
	NodeGuidsPerHash = NodeGuidsPerHash.FilterByPredicate([](const TPair<uint32, TArray<FGuid>>& Pair) -> bool
	{
		return Pair.Value.Num() > 1;
	});

	// now that we have the groupings - let's update the nodes
	TMap<uint32, uint32> CombinedHashToPositionHash;
	for(const TPair<uint32, TArray<FGuid>>& Pair : NodeGuidsPerHash)
	{
		uint32 CombinedGuidHash = 0;
		for(const FGuid& ChildNodeGuid : Pair.Value)
		{
			CombinedGuidHash = HashCombine(CombinedGuidHash, GetTypeHash(ChildNodeGuid));
		}
		CombinedHashToPositionHash.Add(CombinedGuidHash, Pair.Key);
		
		FSchematicGraphGroupNode* GroupNode = nullptr;
		if(FSchematicGraphGroupNode** ExistingGroupNode = GroupNodeByHash.Find(CombinedGuidHash))
		{
			GroupNode = *ExistingGroupNode;
		}
		else
		{
			GroupNode = GraphData->AddAutoGroupNode();
			GroupNodeByHash.Add(CombinedGuidHash, GroupNode);
		}

		const TArray<FGuid> PreviousChildNodeGuids = GroupNode->GetChildNodeGuids();
		const TArray<FGuid>& NextChildNodeGuids = Pair.Value;

		for(const FGuid& NextChildNodeGuid : NextChildNodeGuids)
		{
			if(!PreviousChildNodeGuids.Contains(NextChildNodeGuid))
			{
				GraphData->SetParentNode(NextChildNodeGuid, GroupNode->GetGuid());
			}
		}
	}

	// update the guid based map based on the updated existing group nodes
	GroupNodeGuidByHash.Reset();
	for(const TPair<uint32, FSchematicGraphGroupNode*>& Pair : GroupNodeByHash)
	{
		// remove redundant nodes
		const uint32* PositionHash = CombinedHashToPositionHash.Find(Pair.Key);
		if(PositionHash == nullptr)
		{
			GraphData->RemoveNode(Pair.Value->GetGuid());
			continue;
		}
		const TArray<FGuid>* Guids = NodeGuidsPerHash.Find(*PositionHash); 
		if(Guids == nullptr)
		{
			GraphData->RemoveNode(Pair.Value->GetGuid());
			continue;
		}

		FSchematicGraphNode* Node = Pair.Value;
		GroupNodeGuidByHash.Add(Pair.Key, Node->GetGuid());

		// move the group node to the right location
		const FVector2d AveragePosition = NodePositionPerHash.FindChecked(*PositionHash);

		// The average position is already adjusted for DPI scale
		// The position on the FSchematicGraphNode should not have that correction, we need to undo the scale
		FVector2d AveragePositionWithoutDPIScale = AveragePosition;
		AdjustPositionWithDPIScale(AveragePositionWithoutDPIScale, true);
		Pair.Value->SetPosition(AveragePositionWithoutDPIScale);

		// also update the widget since it has an animated position
		if(const SSchematicGraphNode* Widget = FindNode(Node->GetGuid()))
		{
			Widget->Position->SetValueAndStop(AveragePosition);
			Widget->Scale->SetValueAndStop(1.f);
		}
		
	}
}

void SSchematicGraphPanel::UpdateAutoScalingForNodes()
{
	TSharedPtr<FSchematicGraphModel> GraphData = GraphDataWeak.Pin();
	if(GraphData == nullptr)
	{
		return;
	}
	
	// for now brute force find all neighbors
	// and determine how much of the radius we have
	// to reduce to avoid overlap.
	// todo: use a faster distance algorithm
	TArray<double> RadiusReductionPerNode;
	RadiusReductionPerNode.AddZeroed(Children.Num());
	
	for (int32 i=0; i<Children.Num(); ++i)
	{
		if(!PerNodeCaches[i].bIsAutoScaling)
		{
			continue;
		}
		
		const FVector2d& PositionA = PerNodeCaches[i].Position;
		const double RadiusA = PerNodeCaches[i].Radius;

		for (int32 j=i+1; j<Children.Num(); ++j)
		{
			if(!PerNodeCaches[j].bIsAutoScaling)
			{
				continue;
			}

			const FVector2d& PositionB = PerNodeCaches[j].Position;
			const double RadiusB = PerNodeCaches[j].Radius;
			static constexpr double AutoScalePadding = 4.0;
			const double MinDistance = RadiusA + RadiusB + AutoScalePadding;

			const double Distance = (PositionA - PositionB).Size();
			if(Distance < SMALL_NUMBER || Distance > MinDistance)
			{
				continue;
			}

			const double RadiusReduction = (MinDistance - Distance) * 0.5;
			RadiusReductionPerNode[i] = FMath::Max(RadiusReductionPerNode[i], RadiusReduction);
			RadiusReductionPerNode[j] = FMath::Max(RadiusReductionPerNode[j], RadiusReduction);
		}
	}

	// mark nodes for auto scaling
	for (int32 i=0; i<Children.Num(); ++i)
	{
		TSharedRef<SSchematicGraphNode> Widget = GetChild(i);
		if(RadiusReductionPerNode[i] > SMALL_NUMBER)
		{
			const float Scale = (PerNodeCaches[i].Radius - RadiusReductionPerNode[i]) / PerNodeCaches[i].Radius;
			Widget->AutoScale = FMath::Max(Scale, 0.4f);
		}
		else
		{
			Widget->AutoScale.Reset();
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif