// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SchematicGraphPanel/SchematicGraphModel.h"
#include "SchematicGraphPanel/SSchematicGraphPanel.h"

#define LOCTEXT_NAMESPACE "SchematicGraphModel"

void FSchematicGraphModel::Reset()
{
	Nodes.Reset();
	Links.Reset();

	if (OnGraphResetDelegate.IsBound())
	{
		OnGraphReset().Broadcast();
	}
}

bool FSchematicGraphModel::RemoveNode(const FGuid& InNodeGuid)
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(TTuple<TArray<FGuid>,TArray<FGuid>>* ExistingTuple = NodeGuidToLinkGuids.Find(InNodeGuid))
		{
			TTuple<TArray<FGuid>,TArray<FGuid>> LinksToNode;
			Swap(LinksToNode, *ExistingTuple);
			NodeGuidToLinkGuids.Remove(InNodeGuid);
			for(const FGuid& LinkGuid : LinksToNode.Get<0>())
			{
				(void)RemoveLink(LinkGuid);
			}
			for(const FGuid& LinkGuid : LinksToNode.Get<1>())
			{
				(void)RemoveLink(LinkGuid);
			}
		}

		if(LastExpandedNode == InNodeGuid)
		{
			LastExpandedNode = FGuid();
		}
			
		if (OnNodeRemovedDelegate.IsBound())
		{
			OnNodeRemovedDelegate.Broadcast(Node);
		}

		const TArray<FGuid> ChildNodeGuids = Node->GetChildNodeGuids(); 
		for(const FGuid& ChildNodeGuid : ChildNodeGuids)
		{
			(void)RemoveFromParentNode(ChildNodeGuid);
		}
		
		const FGuid NodeGuid = Node->GetGuid();
		(void)RemoveFromParentNode(NodeGuid);
		NodeByGuid.Remove(Node->GetGuid());
		Nodes.RemoveAll([NodeGuid](const TSharedPtr<FSchematicGraphNode>& ExistingNode) -> bool
		{
			return ExistingNode->GetGuid() == NodeGuid;
		});
		return true;
	}
	return false;
}

bool FSchematicGraphModel::SetParentNode(const FGuid& InChildNodeGuid, const FGuid& InParentNodeGuid, bool bUpdateGroupNode)
{
	if(const FSchematicGraphNode* Node = FindNode(InChildNodeGuid))
	{
		return SetParentNode(Node, FindNode(InParentNodeGuid), bUpdateGroupNode);
	}
	return false;
}

bool FSchematicGraphModel::SetParentNode(const FSchematicGraphNode* InChildNode, const FSchematicGraphNode* InParentNode, bool bUpdateGroupNode)
{
	check(InChildNode);

	if(InParentNode)
	{
		if(InParentNode == InChildNode->GetParentNode())
		{
			return false;
		}
		
		(void)RemoveFromParentNode(InChildNode, bUpdateGroupNode);
		const_cast<FSchematicGraphNode*>(InParentNode)->ChildNodeGuids.AddUnique(InChildNode->GetGuid());
		const_cast<FSchematicGraphNode*>(InChildNode)->ParentNodeGuid = InParentNode->GetGuid();
		return true;
	}
	
	if(const FSchematicGraphNode* CurrentParentNode = InChildNode->GetParentNode())
	{
		FSchematicGraphNode* MutableParentNode = const_cast<FSchematicGraphNode*>(CurrentParentNode);
		if(bUpdateGroupNode && (MutableParentNode->ChildNodeGuids.Num() == 1))
		{
			if(FSchematicGraphGroupNode* GroupParentNode = Cast<FSchematicGraphGroupNode>(MutableParentNode))
			{
				GroupParentNode->SetExpanded(false);
				if(LastExpandedNode == GroupParentNode->GetGuid())
				{
					ClearLastExpandedNode();
				}
			}
		}
		(void)MutableParentNode->ChildNodeGuids.Remove(InChildNode->GetGuid());
		const_cast<FSchematicGraphNode*>(InChildNode)->ParentNodeGuid = FGuid();
		return true;
	}
	
	return false;
}

bool FSchematicGraphModel::RemoveFromParentNode(const FGuid& InChildNodeGuid, bool bUpdateGroupNode)
{
	if(const FSchematicGraphNode* Node = FindNode(InChildNodeGuid))
	{
		return RemoveFromParentNode(Node, bUpdateGroupNode);
	}
	return false;
}

bool FSchematicGraphModel::RemoveFromParentNode(const FSchematicGraphNode* InChildNode, bool bUpdateGroupNode)
{
	check(InChildNode);
	return SetParentNode(InChildNode, nullptr, bUpdateGroupNode);
}

FVector2d FSchematicGraphModel::GetPositionForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetPositionForNode(Node);
	}
	return FVector2d::ZeroVector;
}

FVector2d FSchematicGraphModel::GetPositionForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->GetPosition();
}

FVector2d FSchematicGraphModel::GetPositionOffsetForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetPositionOffsetForNode(Node);
	}
	return FVector2d::ZeroVector;
}

FVector2d FSchematicGraphModel::GetPositionOffsetForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->GetPositionOffset();
}

bool FSchematicGraphModel::GetPositionAnimationEnabledForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetPositionAnimationEnabledForNode(Node);
	}
	return false;
}

bool FSchematicGraphModel::GetPositionAnimationEnabledForNode(const FSchematicGraphNode* InNode) const
{
	return true;
}

FVector2d FSchematicGraphModel::GetSizeForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetSizeForNode(Node);
	}
	return SSchematicGraphNode::DefaultNodeSize;
}

FVector2d FSchematicGraphModel::GetSizeForNode(const FSchematicGraphNode* InNode) const
{
	return SSchematicGraphNode::DefaultNodeSize;
}

float FSchematicGraphModel::GetScaleForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetScaleForNode(Node);
	}
	return 1.f;
}

float FSchematicGraphModel::GetScaleForNode(const FSchematicGraphNode* InNode) const
{
	float Scale = 1.f;

	if(const FSchematicGraphNode* ParentNode = InNode->GetParentNode())
	{
		const TOptional<float> ParentScale = GetScaleForChildNode(ParentNode, InNode);
		if(ParentScale.IsSet())
		{
			Scale *= ParentScale.GetValue();
		}
	}
	
	return Scale * GetScaleOffsetForNode(InNode);
}

float FSchematicGraphModel::GetScaleOffsetForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetScaleOffsetForNode(Node);
	}
	return 1.f;
}

float FSchematicGraphModel::GetScaleOffsetForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->GetScaleOffset();
}

 float FSchematicGraphModel::GetMinimumLinkDistanceForNode(const FGuid& InNodeGuid) const
 {
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetMinimumLinkDistanceForNode(Node);
	}
	return 0;
 }

 float FSchematicGraphModel::GetMinimumLinkDistanceForNode(const FSchematicGraphNode* InNode) const
 {
	check(InNode);

	// return the radius of the node - 1 pixel
	const FVector2d Size = GetSizeForNode(InNode);
	return Size.GetMax() * 0.5f - 1.f;
 }

 bool FSchematicGraphModel::IsAutoScaleEnabledForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return IsAutoScaleEnabledForNode(Node);
	}
	return false;
}

bool FSchematicGraphModel::IsAutoScaleEnabledForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	if(InNode->HasParentNode())
	{
		return false;
	}
	return InNode->IsAutoScaleEnabled();
}

int32 FSchematicGraphModel::GetNumLayersForNode(const FGuid& InNodeGuid) const
 {
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetNumLayersForNode(Node);
	}
	return 0;
}

int32 FSchematicGraphModel::GetNumLayersForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->GetNumLayers();
}

FLinearColor FSchematicGraphModel::GetColorForNode(const FGuid& InNodeGuid, int32 InLayerIndex) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		const FLinearColor Color = GetColorForNode(Node, InLayerIndex);
		if(GetVisibilityForNode(Node) == ESchematicGraphVisibility::FadedOut)
		{
			return Color * 0.5f;
		}
		return Color;
	}
	return FLinearColor::White;
}

FLinearColor FSchematicGraphModel::GetColorForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const
{
	check(InNode);
	return InNode->GetColor(InLayerIndex);
}

const FSlateBrush* FSchematicGraphModel::GetBrushForNode(const FGuid& InNodeGuid, int32 InLayerIndex) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSlateBrush* Brush = GetBrushForNode(Node, InLayerIndex))
		{
			return Brush;
		}
	}

	static const FSlateBrush* DefaultBrush = SSchematicGraphNode::FArguments()._BrushGetter(FGuid(), INDEX_NONE);
	return DefaultBrush;
}

const FSlateBrush* FSchematicGraphModel::GetBrushForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const
{
	check(InNode);
	return InNode->GetBrush(InLayerIndex);
}

FText FSchematicGraphModel::GetToolTipForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetToolTipForNode(Node);
	}
	return FText();
}

FText FSchematicGraphModel::GetToolTipForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->GetToolTip();
}

ESchematicGraphVisibility::Type FSchematicGraphModel::GetVisibilityForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return GetVisibilityForNode(Node);
	}
	return ESchematicGraphVisibility::Visible;
}

ESchematicGraphVisibility::Type FSchematicGraphModel::GetVisibilityForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->GetVisibility();
}

TOptional<ESchematicGraphVisibility::Type> FSchematicGraphModel::GetVisibilityForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const
{
	if(const FSchematicGraphNode* ParentNode = FindNode(InParentNodeGuid))
	{
		if(const FSchematicGraphNode* ChildNode = FindNode(InChildNodeGuid))
		{
			return GetVisibilityForChildNode(ParentNode, ChildNode);
		}
	}
	return TOptional<ESchematicGraphVisibility::Type>();
}

TOptional<ESchematicGraphVisibility::Type> FSchematicGraphModel::GetVisibilityForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const
{
	check(InParentNode);
	check(InChildNode);
	return InParentNode->GetVisibilityForChildNode(InChildNode);
}

TOptional<FVector2d> FSchematicGraphModel::GetPositionForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const
{
	if(const FSchematicGraphNode* ParentNode = FindNode(InParentNodeGuid))
	{
		if(const FSchematicGraphNode* ChildNode = FindNode(InChildNodeGuid))
		{
			return GetPositionForChildNode(ParentNode, ChildNode);
		}
	}
	return TOptional<FVector2d>();
}

TOptional<FVector2d> FSchematicGraphModel::GetPositionForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const
{
	check(InParentNode);
	check(InChildNode);
	return InParentNode->GetPositionForChildNode(InChildNode);
}

TOptional<float> FSchematicGraphModel::GetScaleForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const
{
	if(const FSchematicGraphNode* ParentNode = FindNode(InParentNodeGuid))
	{
		if(const FSchematicGraphNode* ChildNode = FindNode(InChildNodeGuid))
		{
			return GetScaleForChildNode(ParentNode, ChildNode);
		}
	}
	return TOptional<float>();
}

TOptional<float> FSchematicGraphModel::GetScaleForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const
{
	check(InParentNode);
	check(InChildNode);
	return InParentNode->GetScaleForChildNode(InChildNode);
}

TOptional<bool> FSchematicGraphModel::GetInteractivityForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const
{
	if(const FSchematicGraphNode* ParentNode = FindNode(InParentNodeGuid))
	{
		if(const FSchematicGraphNode* ChildNode = FindNode(InChildNodeGuid))
		{
			return GetInteractivityForChildNode(ParentNode, ChildNode);
		}
	}
	return TOptional<bool>();
}

TOptional<bool> FSchematicGraphModel::GetInteractivityForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const
{
	check(InParentNode);
	check(InChildNode);
	return InParentNode->GetInteractivityForChildNode(InChildNode);
}

bool FSchematicGraphModel::IsDragSupportedForNode(const FGuid& InNodeGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		return IsDragSupportedForNode(Node);
	}
	return false;
}

bool FSchematicGraphModel::IsDragSupportedForNode(const FSchematicGraphNode* InNode) const
{
	check(InNode);
	return InNode->IsDragSupported();
}

bool FSchematicGraphModel::GetContextMenuForNode(const FSchematicGraphNode* InNode, FMenuBuilder& OutMenu) const
{
	return false;
}

const TArray<TSharedPtr<FSchematicGraphNode>> FSchematicGraphModel::GetSelectedNodes() const
{
	return Nodes.FilterByPredicate([](const TSharedPtr<FSchematicGraphNode>& Node)
	{
		if (Node.IsValid())
		{
			return Node->IsSelected();
		}
		return false;
	});
}

void FSchematicGraphModel::ClearSelection()
{
	for (TSharedPtr<FSchematicGraphNode> Node : Nodes)
	{
		if (Node.IsValid())
		{
			Node->SetSelected(false);
		}
	}
}

FLinearColor FSchematicGraphModel::GetBackgroundColorForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetBackgroundColorForTag(Tag);
		}
	}
	return FLinearColor::White;
}

FLinearColor FSchematicGraphModel::GetBackgroundColorForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetBackgroundColor();
}

FLinearColor FSchematicGraphModel::GetForegroundColorForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetForegroundColorForTag(Tag);
		}
	}
	return FLinearColor::White;
}

FLinearColor FSchematicGraphModel::GetForegroundColorForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetForegroundColor();
}

FLinearColor FSchematicGraphModel::GetLabelColorForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetLabelColorForTag(Tag);
		}
	}
	return FLinearColor::White;
}

FLinearColor FSchematicGraphModel::GetLabelColorForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetLabelColor();
}

const FSlateBrush* FSchematicGraphModel::GetBackgroundBrushForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetBackgroundBrushForTag(Tag);
		}
	}
	return nullptr;
}

const FSlateBrush* FSchematicGraphModel::GetBackgroundBrushForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetBackgroundBrush();
}

const FSlateBrush* FSchematicGraphModel::GetForegroundBrushForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetForegroundBrushForTag(Tag);
		}
	}
	return nullptr;
}

const FSlateBrush* FSchematicGraphModel::GetForegroundBrushForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetForegroundBrush();
}

const FText FSchematicGraphModel::GetLabelForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetLabelForTag(Tag);
		}
	}
	return FText();
}

const FText FSchematicGraphModel::GetLabelForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetLabel();
}

const FText FSchematicGraphModel::GetToolTipForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetToolTipForTag(Tag);
		}
	}
	return FText();
}

const FText FSchematicGraphModel::GetToolTipForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);
	return InTag->GetToolTip();
}

ESchematicGraphVisibility::Type FSchematicGraphModel::GetVisibilityForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const
{
	if(const FSchematicGraphNode* Node = FindNode(InNodeGuid))
	{
		if(const FSchematicGraphTag* Tag = Node->FindTag(InTagGuid))
		{
			return GetVisibilityForTag(Tag);
		}
	}
	return ESchematicGraphVisibility::Visible;
}

ESchematicGraphVisibility::Type FSchematicGraphModel::GetVisibilityForTag(const FSchematicGraphTag* InTag) const
{
	check(InTag);

	if(InTag->IsA<FSchematicGraphGroupTag>())
	{
		if(const FSchematicGraphAutoGroupNode* AutoGroupNode = Cast<FSchematicGraphAutoGroupNode>(InTag->GetNode()))
		{
			if(AutoGroupNode->GetNumChildNodes() < 3 || AutoGroupNode->IsExpanded())
			{
				return ESchematicGraphVisibility::Hidden;
			}
		}
	}
	
	return InTag->GetVisibility();
}

bool FSchematicGraphModel::IsLinkedTo(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid) const
{
	return FindLink<>(InSourceNodeGuid, InTargetNodeGuid) != nullptr;
}

TArray<const FSchematicGraphLink*> FSchematicGraphModel::FindLinksOnNode(const FGuid& InNodeGuid) const
{
	TArray<const FSchematicGraphLink*> Result;
	for(const TSharedPtr<FSchematicGraphLink>& Link : Links)
	{
		if(Link->GetSourceNodeGuid() == InNodeGuid ||
			Link->GetTargetNodeGuid() == InNodeGuid)
		{
			Result.Add(Link.Get());
		}
	}
	return Result;
}

TArray<const FSchematicGraphLink*> FSchematicGraphModel::FindLinksOnSource(const FGuid& InSourceNodeGuid) const
{
	TArray<const FSchematicGraphLink*> Result;
	for(const TSharedPtr<FSchematicGraphLink>& Link : Links)
	{
		if(Link->GetSourceNodeGuid() == InSourceNodeGuid)
		{
			Result.Add(Link.Get());
		}
	}
	return Result;
}

TArray<const FSchematicGraphLink*> FSchematicGraphModel::FindLinksOnTarget(const FGuid& InTargetNodeGuid) const
{
	TArray<const FSchematicGraphLink*> Result;
	for(const TSharedPtr<FSchematicGraphLink>& Link : Links)
	{
		if(Link->GetTargetNodeGuid() == InTargetNodeGuid)
		{
			Result.Add(Link.Get());
		}
	}
	return Result;
}

bool FSchematicGraphModel::RemoveLink(const FGuid& InLinkGuid)
{
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		if (OnLinkRemovedDelegate.IsBound())
		{
			OnLinkRemovedDelegate.Broadcast(Link);
		}
		const FGuid LinkGuid = Link->GetGuid();
		const uint32 LinkHash = Link->GetLinkHash();
		const FGuid SourceNodeGuid = Link->GetSourceNodeGuid();
		const FGuid TargetNodeGuid = Link->GetTargetNodeGuid();
		LinkByGuid.Remove(Link->GetGuid());
		LinkByHash.Remove(LinkHash);
		if(TTuple<TArray<FGuid>,TArray<FGuid>>* Tuple = NodeGuidToLinkGuids.Find(SourceNodeGuid))
		{
			Tuple->Get<0>().Remove(LinkGuid);
			if(Tuple->Get<0>().IsEmpty() && Tuple->Get<1>().IsEmpty())
			{
				NodeGuidToLinkGuids.Remove(SourceNodeGuid);
			}
		}
		if(TTuple<TArray<FGuid>,TArray<FGuid>>* Tuple = NodeGuidToLinkGuids.Find(TargetNodeGuid))
		{
			Tuple->Get<1>().Remove(LinkGuid);
			if(Tuple->Get<0>().IsEmpty() && Tuple->Get<1>().IsEmpty())
			{
				NodeGuidToLinkGuids.Remove(TargetNodeGuid);
			}
		}

		Links.RemoveAll([LinkGuid](const TSharedPtr<FSchematicGraphLink>& ExistingLink) -> bool
		{
			return ExistingLink->GetGuid() == LinkGuid;
		});
		return true;
	}
	return false;
}

 float FSchematicGraphModel::GetMinimumForLink(const FGuid& InLinkGuid) const
 {
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetMinimumForLink(Link);
	}
	return 0.f;
 }

 float FSchematicGraphModel::GetMinimumForLink(const FSchematicGraphLink* InLink) const
 {
	check(InLink);
	return InLink->GetMinimum();
 }

 float FSchematicGraphModel::GetMaximumForLink(const FGuid& InLinkGuid) const
 {
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetMaximumForLink(Link);
	}
	return 1.f;
 }

 float FSchematicGraphModel::GetMaximumForLink(const FSchematicGraphLink* InLink) const
 {
	check(InLink);
	return InLink->GetMaximum();
 }

FVector2d FSchematicGraphModel::GetSourceNodeOffsetForLink(const FGuid& InLinkGuid) const
 {
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetSourceNodeOffsetForLink(Link);
	}
	return FVector2d::ZeroVector;
 }

 FVector2d FSchematicGraphModel::GetSourceNodeOffsetForLink(const FSchematicGraphLink* InLink) const
 {
	check(InLink);
	return InLink->GetSourceNodeOffset();
 }

 FVector2d FSchematicGraphModel::GetTargetNodeOffsetForLink(const FGuid& InLinkGuid) const
 {
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetTargetNodeOffsetForLink(Link);
	}
	return FVector2d::ZeroVector;
 }

 FVector2d FSchematicGraphModel::GetTargetNodeOffsetForLink(const FSchematicGraphLink* InLink) const
 {
	check(InLink);
	return InLink->GetTargetNodeOffset();
 }

 FLinearColor FSchematicGraphModel::GetColorForLink(const FGuid& InLinkGuid) const
{
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		const FLinearColor Color = GetColorForLink(Link);
		if(GetVisibilityForLink(Link) == ESchematicGraphVisibility::FadedOut)
		{
			return Color * 0.5f;
		}
		return Color;
	}
	return FLinearColor::White;
}

FLinearColor FSchematicGraphModel::GetColorForLink(const FSchematicGraphLink* InLink) const
{
	check(InLink);
	return InLink->GetColor();
}

 float FSchematicGraphModel::GetThicknessForLink(const FGuid& InLinkGuid) const
 {
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetThicknessForLink(Link);
	}
	return 1.f;
 }

 float FSchematicGraphModel::GetThicknessForLink(const FSchematicGraphLink* InLink) const
 {
	check(InLink);
	return InLink->GetThickness();
 }

 const FSlateBrush* FSchematicGraphModel::GetBrushForLink(const FGuid& InLinkGuid) const
{
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetBrushForLink(Link);
	}
	return nullptr;
}

const FSlateBrush* FSchematicGraphModel::GetBrushForLink(const FSchematicGraphLink* InLink) const
{
	check(InLink);
	return InLink->GetBrush();
}

const FText FSchematicGraphModel::GetToolTipForLink(const FGuid& InLinkGuid) const
{
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetToolTipForLink(Link);
	}
	return FText();
}

const FText FSchematicGraphModel::GetToolTipForLink(const FSchematicGraphLink* InLink) const
{
	check(InLink);
	return InLink->GetToolTip();
}

ESchematicGraphVisibility::Type FSchematicGraphModel::GetVisibilityForLink(const FGuid& InLinkGuid) const
{
	if(const FSchematicGraphLink* Link = FindLink(InLinkGuid))
	{
		return GetVisibilityForLink(Link);
	}
	return ESchematicGraphVisibility::Visible;
}

ESchematicGraphVisibility::Type FSchematicGraphModel::GetVisibilityForLink(const FSchematicGraphLink* InLink) const
{
	check(InLink);
	return InLink->GetVisibility();
}

FSchematicGraphGroupNode* FSchematicGraphModel::AddAutoGroupNode()
{
	return AddNode<FSchematicGraphAutoGroupNode>(); 
}

const FSchematicGraphGroupNode* FSchematicGraphModel::GetLastExpandedNode() const
{
	if(LastExpandedNode.IsValid())
	{
		return FindNode<FSchematicGraphGroupNode>(LastExpandedNode);
	}
	return nullptr;
}

void FSchematicGraphModel::SetLastExpandedNode(const FSchematicGraphGroupNode* InGroupNode)
{
	LastExpandedNode = InGroupNode ? InGroupNode->GetGuid() : FGuid();
}

void FSchematicGraphModel::Tick(float InDeltaTime)
{
	if(LastExpandedNode.IsValid())
	{
		if(const FSchematicGraphGroupNode* GroupNode = GetLastExpandedNode())
		{
			if(!GroupNode->IsExpanded())
			{
				LastExpandedNode = FGuid();
			}
		}
		else
		{
			LastExpandedNode = FGuid();
		}
	}

	// remove invalid parent / child relationships
	for(const TSharedPtr<FSchematicGraphNode>& Node : Nodes)
	{
		if(Node->ParentNodeGuid.IsValid())
		{
			if(!NodeByGuid.Contains(Node->ParentNodeGuid))
			{
				Node->ParentNodeGuid = FGuid();
			}
		}

		const FGuid Guid = Node->GetGuid();
		Node->ChildNodeGuids.RemoveAll([this, Guid](const FGuid& InChildGuid) -> bool
		{
			if(const FSchematicGraphNode* ChildNode = FindNode(InChildGuid))
			{
				return ChildNode->ParentNodeGuid != Guid;
			}
			return true;
		});
	}
}

#undef LOCTEXT_NAMESPACE

#endif