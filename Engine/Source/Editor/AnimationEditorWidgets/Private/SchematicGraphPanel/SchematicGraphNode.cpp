// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SchematicGraphPanel/SchematicGraphNode.h"
#include "SchematicGraphPanel/SchematicGraphModel.h"
#include <SchematicGraphPanel/SchematicGraphStyle.h>
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SchematicGraphNode"

FSchematicGraphNode::FSchematicGraphNode()
{
	static const FSlateBrush* BackgroundBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Background");
	static const FSlateBrush* OutlineBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Outline.Single");
	static const FSlateBrush* DotBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Dot.Small");
	Brushes = { BackgroundBrush, OutlineBrush, DotBrush };
	Colors = { FLinearColor::White * 0.4f, FLinearColor::White, FLinearColor::Blue };
}

FSchematicGraphNode* FSchematicGraphNode::GetParentNode()
{
	if(Model && ParentNodeGuid.IsValid())
	{
		return Model->FindNode(ParentNodeGuid);
	}
	return nullptr;
}

const FSchematicGraphNode* FSchematicGraphNode::GetParentNode() const
{
	return const_cast<FSchematicGraphNode*>(this)->GetParentNode();
}

const FGuid& FSchematicGraphNode::GetRootNodeGuid() const
{
	if(const FSchematicGraphNode* ParentNode = GetParentNode())
	{
		return ParentNode->GetRootNodeGuid();
	}
	return GetGuid();
}

FSchematicGraphNode* FSchematicGraphNode::GetRootNode()
{
	if(Model)
	{
		return Model->FindNode(GetRootNodeGuid());
	}
	return nullptr;
}

const FSchematicGraphNode* FSchematicGraphNode::GetRootNode() const
{
	return const_cast<FSchematicGraphNode*>(this)->GetRootNode();
}

FSchematicGraphGroupNode* FSchematicGraphNode::GetGroupNode() 
{
	return Cast<FSchematicGraphGroupNode>(GetParentNode());
}

const FSchematicGraphGroupNode* FSchematicGraphNode::GetGroupNode() const
{
	return const_cast<FSchematicGraphNode*>(this)->GetGroupNode();
}

const FSchematicGraphNode* FSchematicGraphNode::GetChildNode(int32 InChildNodeIndex) const
{
	return const_cast<FSchematicGraphNode*>(this)->GetChildNode(InChildNodeIndex);
}

FSchematicGraphNode* FSchematicGraphNode::GetChildNode(int32 InChildNodeIndex)
{
	check(ChildNodeGuids.IsValidIndex(InChildNodeIndex));

	const FGuid& ChildNodeGuid = ChildNodeGuids[InChildNodeIndex];
	if(Model && ChildNodeGuid.IsValid())
	{
		return Model->FindNode(ChildNodeGuid);
	}
	return nullptr;
}

TOptional<ESchematicGraphVisibility::Type> FSchematicGraphNode::GetVisibilityForChildNode(const FGuid& InChildGuid) const
{
	if(Model)
	{
		if(const FSchematicGraphNode* ChildNode = Model->FindNode(InChildGuid))
		{
			return GetVisibilityForChildNode(ChildNode);
		}
	}
	return TOptional<ESchematicGraphVisibility::Type>();
}

TOptional<FVector2d> FSchematicGraphNode::GetPositionForChildNode(const FGuid& InChildGuid) const
{
	if(Model)
	{
		if(const FSchematicGraphNode* ChildNode = Model->FindNode(InChildGuid))
		{
			return GetPositionForChildNode(ChildNode);
		}
	}
	return TOptional<FVector2d>();
}

TOptional<float> FSchematicGraphNode::GetScaleForChildNode(const FGuid& InChildGuid) const
{
	if(Model)
	{
		if(const FSchematicGraphNode* ChildNode = Model->FindNode(InChildGuid))
		{
			return GetScaleForChildNode(ChildNode);
		}
	}
	return TOptional<float>();
}

TOptional<bool> FSchematicGraphNode::GetInteractivityForChildNode(const FGuid& InChildGuid) const
{
	if(Model)
	{
		if(const FSchematicGraphNode* ChildNode = Model->FindNode(InChildGuid))
		{
			return GetInteractivityForChildNode(ChildNode);
		}
	}
	return TOptional<bool>();
}

FVector2d FSchematicGraphNode::GetPosition() const
{
	if(const FSchematicGraphNode* ParentNode = GetParentNode())
	{
		const TOptional<FVector2d> ParentPosition = Model->GetPositionForChildNode(ParentNode, this);
		if(ParentPosition.IsSet())
		{
			return ParentPosition.GetValue();
		}
	}
	return Position;
}

const FText& FSchematicGraphNode::GetToolTip() const
{
	return ToolTip;
}

ESchematicGraphVisibility::Type FSchematicGraphNode::GetVisibility() const
{
	if(Visibility == ESchematicGraphVisibility::Hidden)
	{
		return Visibility;
	}

	const FSchematicGraphNode* ParentNode = GetParentNode();
	
	if(const FSchematicGraphGroupNode* LastExpandedNode = Model->GetLastExpandedNode())
	{
		if(LastExpandedNode != this && LastExpandedNode != ParentNode)
		{
			return ESchematicGraphVisibility::FadedOut;
		}
	}
	
	if(ParentNode)
	{
		const TOptional<ESchematicGraphVisibility::Type> ParentVisibility = Model->GetVisibilityForChildNode(ParentNode, this);
		if(ParentVisibility.IsSet())
		{
			return ParentVisibility.GetValue();
		}
	}
	return Visibility;
}

bool FSchematicGraphNode::IsInteractive() const
{
	if(GetVisibility() != ESchematicGraphVisibility::Visible)
	{
		return false;
	}
	
	if(const FSchematicGraphNode* ParentNode = GetParentNode())
	{
		const TOptional<bool> ParentInteractivity = Model->GetInteractivityForChildNode(ParentNode, this);
		if(ParentInteractivity.IsSet())
		{
			return ParentInteractivity.GetValue();
		}
	}
	return true;
}

FReply FSchematicGraphNode::OnClicked(const FPointerEvent& InMouseEvent)
{
	return FReply::Unhandled();
}

void FSchematicGraphNode::OnMouseEnter()
{
	SetScaleOffset(ScaledUp);
}

void FSchematicGraphNode::OnMouseLeave()
{
	SetScaleOffset(1.f);
}

void FSchematicGraphNode::OnDragOver()
{
	SetScaleOffset(ScaledDown);

	if(FSchematicGraphGroupNode* GroupNode = const_cast<FSchematicGraphGroupNode*>(GetGroupNode()))
	{
		GroupNode->SetExpanded(true);
	}
}

void FSchematicGraphNode::OnDragLeave()
{
	SetScaleOffset(1.f);

	if(FSchematicGraphGroupNode* GroupNode = const_cast<FSchematicGraphGroupNode*>(GetGroupNode()))
	{
		GroupNode->SetExpanded(false);
	}
}

FString FSchematicGraphNode::GetDragDropDecoratorLabel() const
{
	return GetGuid().ToString();
}

bool FSchematicGraphNode::RemoveTag(const FGuid& InTagGuid)
{
	if(const FSchematicGraphTag* Tag = FindTag(InTagGuid))
	{
		if (Model && Model->OnTagRemovedDelegate.IsBound())
		{
			Model->OnTagRemovedDelegate.Broadcast(this, Tag);
		}
		const FGuid TagGuid = Tag->GetGuid();
		TagByGuid.Remove(Tag->GetGuid());
		Tags.RemoveAll([TagGuid](const TSharedPtr<FSchematicGraphTag>& ExistingTag) -> bool
		{
			return ExistingTag->GetGuid() == TagGuid;
		});
		return true;
	}
	return false;
}

void FSchematicGraphNode::NotifyTagAdded(const TSharedPtr<FSchematicGraphTag>& Tag)
{
	if (Model != nullptr && Model->OnTagAddedDelegate.IsBound())
	{
		Model->OnTagAddedDelegate.Broadcast(this, Tag.Get());
	}
}

FSchematicGraphGroupNode::FSchematicGraphGroupNode()
	: AnimationSettings(EEasingInterpolatorType::CubicEaseOut, 0.35f)
{
	static const FSlateBrush* BackgroundBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Background");
	static const FSlateBrush* OutlineBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Outline.Single");
	static const FSlateBrush* GroupBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Dot.Group");
	Brushes = { BackgroundBrush, OutlineBrush, GroupBrush };
	Colors = { FLinearColor::White * 0.4f, FLinearColor::White, FLinearColor::Gray };

	ExpansionState = TAnimatedAttribute<float>::Create(AnimationSettings, 0.f);
	(void)AddTag<FSchematicGraphGroupTag>();
}

bool FSchematicGraphGroupNode::IsExpanded() const
{
	return GetExpansionState() > SMALL_NUMBER;
}

bool FSchematicGraphGroupNode::IsExpanding() const
{
	return (ExpansionState->Get() < 1.f - SMALL_NUMBER) &&
		(ExpansionState->GetDesiredValue() >= 1.f - SMALL_NUMBER);
}

bool FSchematicGraphGroupNode::IsCollapsing() const
{
	return (ExpansionState->Get() >= SMALL_NUMBER) &&
		(ExpansionState->GetDesiredValue() < SMALL_NUMBER);
}

float FSchematicGraphGroupNode::GetExpansionState() const
{
	if(!ExpansionState.IsValid())
	{
		return 0.f;
	}
	return ExpansionState->Get();
}

void FSchematicGraphGroupNode::SetExpanded(bool InExpanded, bool bAutoCloseParentGroups)
{
	if(InExpanded)
	{
		if(GetNumChildNodes() == 0)
		{
			return;
		}
	}
	
	if(!InExpanded && GetExpansionState() < SMALL_NUMBER)
	{
		ExpansionState->SetValueAndStop(0.f);
		LastExpansionState.Reset();
	}
	else
	{
		if(InExpanded && !ExpansionState->IsPlaying())
		{
			// set the state to an initial value so that IsExpanded will return correctly
			if(ExpansionState->Get() < SMALL_NUMBER)
			{
				ExpansionState->SetValueAndStop(SMALL_NUMBER * 2.f);
			}
		}
		
		if(!LastExpansionState.IsSet() || (LastExpansionState.Get(InExpanded) != InExpanded))
		{
			if(!ExpansionState->GetDelay().IsSet())
			{
				ExpansionState->SetDelayOneShot(GetDelayDuration(InExpanded));
			}
			LastExpansionState = InExpanded;
		}
		ExpansionState->Set(InExpanded ? 1.f : 0.f);
	}

	if(InExpanded || bAutoCloseParentGroups)
	{
		if(FSchematicGraphGroupNode* GroupNode = const_cast<FSchematicGraphGroupNode*>(GetGroupNode()))
		{
			GroupNode->SetExpanded(InExpanded, bAutoCloseParentGroups);
		}
	}
	
	if(InExpanded)
	{
		Model->SetLastExpandedNode(this);
	}
	else if(const FSchematicGraphGroupNode* GroupNode = GetGroupNode())
	{
		if(GroupNode->IsExpanded())
		{
			Model->SetLastExpandedNode(GroupNode);
		}
	}

	// also collapse all child nodes
	if(!InExpanded)
	{
		for(int32 ChildIndex = 0; ChildIndex < GetNumChildNodes(); ChildIndex++)
		{
			if(FSchematicGraphGroupNode* ChildNode = Cast<FSchematicGraphGroupNode>(GetChildNode(ChildIndex)))
			{
				ChildNode->SetExpanded(false);
			}
		}
	}
}

TOptional<ESchematicGraphVisibility::Type> FSchematicGraphGroupNode::GetVisibilityForChildNode(const FSchematicGraphNode* InChildNode) const
{
	if(!IsExpanded())
	{
		return ESchematicGraphVisibility::Hidden;
	}
	return Super::GetVisibilityForChildNode(InChildNode);
}

TOptional<FVector2d> FSchematicGraphGroupNode::GetPositionForChildNode(const FSchematicGraphNode* InChildNode) const
{
	if(IsExpanded() && Model)
	{
		const int32 ChildNodeIndex = ChildNodeGuids.Find(InChildNode->GetGuid());
		if(ChildNodeIndex != INDEX_NONE)
		{
			const float Distance = ExpansionRadius * GetExpansionState();
			const float Angle = 360.f * float(ChildNodeIndex) / float(ChildNodeGuids.Num());
			const FVector2d Offset = FVector2d(Distance, 0).GetRotated(Angle);
			return Model->GetPositionForNode(this) + Offset;
		}
	}
	return Super::GetPositionForChildNode(InChildNode);
}

TOptional<float> FSchematicGraphGroupNode::GetScaleForChildNode(const FSchematicGraphNode* InChildNode) const
{
	if(IsExpanded() && Model)
	{
		return GetExpansionState();
	}
	return Super::GetScaleForChildNode(InChildNode);
}

TOptional<bool> FSchematicGraphGroupNode::GetInteractivityForChildNode(const FSchematicGraphNode* InChildNode) const
{
	if(GetExpansionState() < 1.f - SMALL_NUMBER)
	{
		return false;
	}
	return Super::GetInteractivityForChildNode(InChildNode);
}

FReply FSchematicGraphGroupNode::OnClicked(const FPointerEvent& InMouseEvent)
{
	if(GetNumChildNodes() > 0)
	{
		if(!InMouseEvent.GetModifierKeys().AnyModifiersDown())
		{
			SetExpanded(!IsExpanded());
		}
		return FReply::Handled();
	}
	return FSchematicGraphNode::OnClicked(InMouseEvent);
}

void FSchematicGraphGroupNode::OnDragOver()
{
	Super::OnDragOver();
	SetExpanded(true, true);
}

void FSchematicGraphGroupNode::OnDragLeave()
{
	Super::OnDragLeave();
	SetExpanded(false, true);
}

float FSchematicGraphGroupNode::GetDelayDuration(bool bEnter) const
{
	if (FSlateApplication::Get().IsDragDropping())
	{
		return bEnter ? EnterDelayDuration : LeaveDelayDuration;
	}
	return 0.f;
}

void FSchematicGraphGroupNode::SetAnimationSettings(const TEasingAttributeInterpolator<float>::FSettings& InSettings)
{
	ExpansionState = TAnimatedAttribute<float>::Create(AnimationSettings, IsExpanded() ? 1.f : 0.f);
}

const FText& FSchematicGraphAutoGroupNode::GetLabel() const
{
	const FText& SuperLabel = FSchematicGraphGroupNode::GetLabel();
	if(SuperLabel.IsEmpty() && !IsExpanded())
	{
		AutoGroupLabel = FText();

		for(int32 Index = 0; Index < GetNumChildNodes(); Index++)
		{
			if(const FSchematicGraphNode* ChildNode = GetChildNode(Index))
			{
				const FText& ChildLabel = ChildNode->GetLabel();
				if(!ChildLabel.IsEmpty())
				{
					if(!AutoGroupLabel.IsEmpty())
					{
						AutoGroupLabel = FText::Format(LOCTEXT("AutoGroupLabelAppendFormat", "{0}\n{1}"), AutoGroupLabel, ChildLabel);
					}
					else
					{
						AutoGroupLabel = ChildLabel;
					}
				}
			}
		}

		return AutoGroupLabel;
	}
	return SuperLabel;
}

#undef LOCTEXT_NAMESPACE

#endif