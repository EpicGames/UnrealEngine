// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SchematicGraphPanel/SchematicGraphTag.h"
#include "SchematicGraphPanel/SchematicGraphNode.h"
#include "SchematicGraphPanel/SchematicGraphStyle.h"

#define LOCTEXT_NAMESPACE "SchematicGraphTag"

FSchematicGraphTag::FSchematicGraphTag()
{
	static const FSlateBrush* CircleBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Tag.Background");
	BackgroundBrush = CircleBrush;
}

const FSchematicGraphNode* FSchematicGraphTag::GetNode() const
{
	return Node;
}

ESchematicGraphVisibility::Type FSchematicGraphGroupTag::GetVisibility() const
{
	if(const FSchematicGraphNode* GroupNode = Cast<FSchematicGraphNode>(GetNode()))
	{
		return GroupNode->GetNumChildNodes() == 0 ? ESchematicGraphVisibility::Hidden : ESchematicGraphVisibility::Visible;
	}
	return Super::GetVisibility();
}

FText FSchematicGraphGroupTag::GetLabel() const
{
	if(const FSchematicGraphNode* GroupNode = Cast<FSchematicGraphNode>(GetNode()))
	{
		return FText::FromString(FString::FromInt(GroupNode->GetNumChildNodes()));
	}
	return Super::GetLabel();
}

#undef LOCTEXT_NAMESPACE

#endif