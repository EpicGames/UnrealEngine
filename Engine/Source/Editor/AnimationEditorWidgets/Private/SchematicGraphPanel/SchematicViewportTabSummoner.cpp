// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR

#include "SchematicGraphPanel/SchematicViewportTabSummoner.h"
#include "SchematicGraphPanel/SSchematicGraphPanel.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "SchematicViewportTabSummoner"

const FName FSchematicViewportTabSummoner::TabID(TEXT("SchematicViewport"));

FSchematicViewportTabSummoner::FSchematicViewportTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const FSchematicViewportArgs& InArgs)
	: FWorkflowTabFactory(TabID, InHostingApp)
	, SchematicGraphWeak(InArgs.SchematicGraphWeak)
	, OnViewportCreated(InArgs.OnViewportCreated)
{
	TabLabel = LOCTEXT("SchematicViewportTabLabel", "Schematic Viewport");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");

	ViewMenuDescription = LOCTEXT("SchematicViewport_ViewMenu_Desc", "Schematic Viewport");
	ViewMenuTooltip = LOCTEXT("SchematicViewport_ViewMenu_ToolTip", "Show the Schematic Viewport tab");
}

FTabSpawnerEntry& FSchematicViewportTabSummoner::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* CurrentApplicationMode) const
{
	FTabSpawnerEntry& SpawnerEntry = FWorkflowTabFactory::RegisterTabSpawner(InTabManager, CurrentApplicationMode);

	SpawnerEntry.SetReuseTabMethod(FOnFindTabToReuse::CreateLambda([](const FTabId& InTabId) ->TSharedPtr<SDockTab> {
	
		return TSharedPtr<SDockTab>();

	}));

	return SpawnerEntry;
}

TSharedRef<SWidget> FSchematicViewportTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SSchematicGraphPanel> Viewport = SNew(SSchematicGraphPanel)
											.GraphDataModel(SchematicGraphWeak)
											.IsOverlay(false)
											.PaddingLeft(30)
											.PaddingRight(30)
											.PaddingTop(30)
											.PaddingBottom(30)
											.PaddingInterNode(5);
	OnViewportCreated.ExecuteIfBound(Viewport);
	return Viewport;
}

#undef LOCTEXT_NAMESPACE 
#endif