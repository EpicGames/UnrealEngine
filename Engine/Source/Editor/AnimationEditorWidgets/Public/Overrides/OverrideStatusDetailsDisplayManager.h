//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailsDisplayManager.h"
#include "DetailsViewStyleKey.h"
#include "Overrides/OverrideStatusDetailsWidgetBuilder.h"
#include "Overrides/OverrideStatusWidgetMenuBuilder.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class FOverrideStatusDetailsWidgetBuilder;

/**
 * The display manager is used to determine how the details view should behave when using an object
 * filter. In this case the display manager is used to set the property-updated-widget (the override status widget).
 */
class FOverrideStatusDetailsDisplayManager : public FDetailsDisplayManager
{
public:

	UE_API virtual ~FOverrideStatusDetailsDisplayManager() override;

	/**
	 * Returns a @code bool @endcode indicating whether this @code DetailsViewObjectFilter @endcode instance
	 * has a category menu
	 */
	UE_API virtual bool ShouldShowCategoryMenu() override;

	/**
	 * Returns the @code const FDetailsViewStyleKey& @endcode that is the Key to the current FDetailsViewStyle style
	 */
	UE_API virtual const FDetailsViewStyleKey& GetDetailsViewStyleKey() const override;

	// Returns true if this manager can construct the property updated widget
	UE_API virtual bool CanConstructPropertyUpdatedWidgetBuilder() const override;

	// Returns the builder used to construct the property updated widgets, in this case the SOverrideStatusWidget 
	UE_API virtual TSharedPtr<FPropertyUpdatedWidgetBuilder> ConstructPropertyUpdatedWidgetBuilder(const FConstructPropertyUpdatedWidgetBuilderArgs& Args) override;

	// Returns the preconfigured menu builder for this display manager and a given subject
	UE_API TSharedPtr<FOverrideStatusWidgetMenuBuilder> GetMenuBuilder(const FOverrideStatusSubject& InSubject) const;

	FOverrideStatus_CanCreateWidget& OnCanCreateWidget() { return CanCreateWidgetDelegate; }
	FOverrideStatus_GetStatus& OnGetStatus() { return GetStatusDelegate; }
	FOverrideStatus_OnWidgetClicked& OnWidgetClicked() { return WidgetClickedDelegate; }
	FOverrideStatus_OnGetMenuContent& OnGetMenuContent() { return GetMenuContentDelegate; }
	FOverrideStatus_AddOverride& OnAddOverride() { return AddOverrideDelegate; }
	FOverrideStatus_ClearOverride& OnClearOverride() { return ClearOverrideDelegate; }
	FOverrideStatus_ResetToDefault& OnResetToDefault() { return ResetToDefaultDelegate; }
	FOverrideStatus_ValueDiffersFromDefault& OnValueDiffersFromDefault() { return ValueDiffersFromDefaultDelegate; }

private:
	
	UE_API TSharedPtr<FOverrideStatusDetailsWidgetBuilder> ConstructOverrideWidgetBuilder(const FConstructPropertyUpdatedWidgetBuilderArgs& Args);
	UE_API void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void SetIsDisplayingOverrideableObject(bool InIsDisplayingOverrideableObject);

	bool bIsDisplayingOverrideableObject = false;
	
	FExecuteAction InvalidateCachedState;

	FOverrideStatus_CanCreateWidget CanCreateWidgetDelegate;
	FOverrideStatus_GetStatus GetStatusDelegate;
	FOverrideStatus_OnWidgetClicked WidgetClickedDelegate;
	FOverrideStatus_OnGetMenuContent GetMenuContentDelegate;
	FOverrideStatus_AddOverride AddOverrideDelegate;
	FOverrideStatus_ClearOverride ClearOverrideDelegate;
	FOverrideStatus_ResetToDefault ResetToDefaultDelegate;
	FOverrideStatus_ValueDiffersFromDefault ValueDiffersFromDefaultDelegate;

	friend class FOverrideStatusDetailsViewObjectFilter;
};

#undef UE_API
