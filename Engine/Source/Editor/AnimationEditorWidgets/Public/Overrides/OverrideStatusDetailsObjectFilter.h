// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Overrides/OverrideStatusDetailsDisplayManager.h"
#include "Overrides/OverrideStatusWidgetMenuBuilder.h"
#include "DetailsViewObjectFilter.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOverrideStatusObjectFilter_CanMergeObjects, const UObject*, const UObject*);

/**
 * An object filter for the property editor / details view.
 * The filter can decide if it can display a certain object - in this case the object filter
 * is used to provide the override status widget instead of the reset value arrow to the details
 * panel.
 */
class FOverrideStatusDetailsViewObjectFilter : public FDetailsViewObjectFilter
{	
public:
	// the standard method to create an object filter. 
	template<typename T = FOverrideStatusDetailsViewObjectFilter>
	static TSharedPtr<T> Create()
	{
		TSharedPtr<T> ObjectFilter = MakeShared<T>();
		ObjectFilter->InitializeDisplayManager();
		return ObjectFilter;
	}

	// default constructor
	UE_API FOverrideStatusDetailsViewObjectFilter();

	// sets up the  display manager for this filter
	UE_API virtual void InitializeDisplayManager();

	// Given a const TArray<UObject*>& SourceObjects, filters the objects and puts the objects which should
	// be shown in the Details panel in the return TArray<FDetailsViewObjectRoot>. These may be some part of the
	// original SourceObjects array, itself, or it may be some contained sub-objects within the SourceObjects.
	UE_API virtual TArray<FDetailsViewObjectRoot> FilterObjects(const TArray<UObject*>& SourceObjects) override;

	// Returns a preconfigured menu builder for this filter.
	UE_API TSharedPtr<FOverrideStatusWidgetMenuBuilder> GetMenuBuilder(const FOverrideStatusSubject& InSubject) const;
	
	UE_API FOverrideStatus_CanCreateWidget& OnCanCreateWidget();
	UE_API FOverrideStatus_GetStatus& OnGetStatus();
	UE_API FOverrideStatus_OnWidgetClicked& OnWidgetClicked();
	UE_API FOverrideStatus_OnGetMenuContent& OnGetMenuContent();
	UE_API FOverrideStatus_AddOverride& OnAddOverride();
	UE_API FOverrideStatus_ClearOverride& OnClearOverride();
	UE_API FOverrideStatus_ResetToDefault& OnResetToDefault();
	UE_API FOverrideStatus_ValueDiffersFromDefault& OnValueDiffersFromDefault();
	FOverrideStatusObjectFilter_CanMergeObjects& OnCanMergeObjects() { return CanMergeObjectDelegate; }

	static UE_API bool MergeObjectByClass(const UObject* InObjectA, const UObject* InObjectB);

private:
	
	/**
	 * The @code FOverrideStatusDetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedPtr<FOverrideStatusDetailsDisplayManager> OverrideStatusDisplayManager;

	FOverrideStatusObjectFilter_CanMergeObjects CanMergeObjectDelegate;
};

#undef UE_API
