//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "UserInterface/Widgets/PropertyUpdatedWidgetBuilder.h"
#include "OverrideStatusSubject.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class FOverrideStatusDetailsDisplayManager;

/**
 * A Display builder for the override status combo button
 */
class FOverrideStatusDetailsWidgetBuilder : public FPropertyUpdatedWidgetBuilder
{
public:
	DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FOnGetWidget);

public:

	/**
	 * The constructor, which takes a @code TSharedRef<FDetailsDisplayManager> @endcode to initialize
	 * the Details Display Manager
	 *
	 * @param InDetailsDisplayManager the FDetailsDisplayManager which manages the details display
	 * than a property row   
	 */
	UE_API FOverrideStatusDetailsWidgetBuilder(
		const TSharedRef<FOverrideStatusDetailsDisplayManager>& InDetailsDisplayManager,
		const TArray<FOverrideStatusObject>& InObjects,
		const TSharedPtr<const FPropertyPath>& InPropertyPath,
		const FName& InCategory);

	/**
	 * Implements the generation of the Category Menu button SWidget
	 */
	UE_API virtual TSharedPtr<SWidget> GenerateWidget() override;

	/**
	 * Converts this into the SWidget it builds
	 */
	UE_API virtual ~FOverrideStatusDetailsWidgetBuilder() override;

	UE_API FOverrideStatus_CanCreateWidget& OnCanCreateWidget();
	UE_API FOverrideStatus_GetStatus& OnGetStatus();
	UE_API FOverrideStatus_OnWidgetClicked& OnWidgetClicked();
	UE_API FOverrideStatus_OnGetMenuContent& OnGetMenuContent();
	UE_API FOverrideStatus_AddOverride& OnAddOverride();
	UE_API FOverrideStatus_ClearOverride& OnClearOverride();
	UE_API FOverrideStatus_ResetToDefault& OnResetToDefault();
	UE_API FOverrideStatus_ValueDiffersFromDefault& OnValueDiffersFromDefault();

private:

	/**
	 * The @code DetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedRef<FOverrideStatusDetailsDisplayManager> DisplayManager;

	FOverrideStatusSubject Subject;
};

#undef UE_API
