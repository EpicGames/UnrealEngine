//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "SOverrideStatusWidget.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class FPropertyPath;
struct FOverrideStatusSubject;

DECLARE_DELEGATE_RetVal_OneParam(bool, FOverrideStatus_CanCreateWidget, const FOverrideStatusSubject&);
DECLARE_DELEGATE_RetVal_OneParam(EOverrideWidgetStatus::Type, FOverrideStatus_GetStatus, const FOverrideStatusSubject&);
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOverrideStatus_OnWidgetClicked, const FOverrideStatusSubject&, EOverrideWidgetStatus::Type);
DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOverrideStatus_OnGetMenuContent, const FOverrideStatusSubject&, EOverrideWidgetStatus::Type);
DECLARE_DELEGATE_RetVal_OneParam(FReply, FOverrideStatus_AddOverride, const FOverrideStatusSubject&);
DECLARE_DELEGATE_RetVal_OneParam(FReply, FOverrideStatus_ClearOverride, const FOverrideStatusSubject&);
DECLARE_DELEGATE_RetVal_OneParam(FReply, FOverrideStatus_ResetToDefault, const FOverrideStatusSubject&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOverrideStatus_ValueDiffersFromDefault, const FOverrideStatusSubject&);

/**
 * A template handle pointing to given object type
 */
template<typename T>
struct FOverrideStatusObjectHandle
{
public:

	FOverrideStatusObjectHandle()
		: Object(nullptr)
		, Key(NAME_None)
	{
	}
	
	FOverrideStatusObjectHandle(const T* InObject, const FName& InKey = NAME_None)
		: Object(InObject)
		, Key(InKey)
	{
	}

	bool IsValid() const
	{
		return Object != nullptr;
	}

	operator bool() const
	{
		return IsValid();
	}

	const T* GetObject() const
	{
		return Object;
	}
	
	const FName& GetKey() const
	{
		return Key;
	}

	bool operator==(const FOverrideStatusObjectHandle& InOther) const
	{
		return (Object == InOther.Object) && (Key == InOther.Key);
	}

	const T* operator->() const
	{
		return GetObject();
	}

private:
	const T* Object;
	FName Key;
};

/**
 * A single object used within an override widget - the object is identified by a weak object pointer and a potential sub object key.
 */
struct FOverrideStatusObject
{
public:

	UE_API FOverrideStatusObject();
	UE_API FOverrideStatusObject(const UObject* InObject, const FName& InKey = NAME_None);

	UE_API FName GetFName() const;
	UE_API bool IsValid() const;
	UE_API const UObject* GetObject() const;
	UE_API const FName& GetKey() const;

	UE_API bool operator==(const FOverrideStatusObject& InOther) const;

	template<typename T>
	FOverrideStatusObjectHandle<T> GetHandle() const
	{
		return FOverrideStatusObjectHandle<T>(Cast<T>(GetObject()), GetKey());
	}

private:
	TWeakObjectPtr<const UObject> WeakObjectPtr;
	FName Key;
};

/**
 * The subject of an override status (widget).
 * To support multi selection the subject is represented by an array of objects and a property path.
 * This class also offers helper template functions to facilitate the interaction between the list
 * of subject object and the user interface layer.
 */
struct FOverrideStatusSubject
{
public:
	UE_API FOverrideStatusSubject(const TArray<FOverrideStatusObject>& InObjects, const TSharedPtr<const FPropertyPath>& InPropertyPath, const FName& InCategory);
	UE_API FOverrideStatusSubject(const FOverrideStatusObject& InObject, const TSharedPtr<const FPropertyPath>& InPropertyPath = nullptr, const FName& InCategory = NAME_None);
	UE_API FOverrideStatusSubject(const UObject* InObject, const TSharedPtr<const FPropertyPath>& InPropertyPath = nullptr, const FName& InCategory = NAME_None, const FName& InKey = NAME_None);

	// returns true if any of the objects within the subject is still valid
	UE_API bool IsValid() const;

	// returns the number of objects
	UE_API int32 Num() const;

	// array access operator 
	UE_API const FOverrideStatusObject& operator [](int32 InIndex) const;

	// array handle access
	template<typename T>
	FOverrideStatusObjectHandle<T> GetHandle(int32 InIndex) const
	{
		return operator[](InIndex).GetHandle<T>();
	}

	// range access
	TArray<FOverrideStatusObject>::RangedForConstIteratorType begin() const { return Objects.begin(); }
	TArray<FOverrideStatusObject>::RangedForConstIteratorType end() const { return Objects.end(); }

	// returns true if this subject contains a property path
	UE_API bool HasPropertyPath() const;

	// getter for the property path
	UE_API const TSharedPtr<const FPropertyPath>& GetPropertyPath() const;

	// getter for the propery path string (uses a cache inside and is faster than GetPropertyPath()->ToString())
	UE_API const FString& GetPropertyPathString(const TCHAR* Separator = TEXT("->")) const;

	// returns true if this subject is linked to a category
	UE_API bool HasCategory() const;

	// getter for category
	UE_API const FName& GetCategory() const;

	// returns true if a given object is part of the subject
	UE_API bool Contains(const FOverrideStatusObject& InObject) const;

	// returns true if the subject contains an object of type ObjectType
	template<typename ObjectType>
	bool Contains() const
	{
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(Cast<ObjectType>(operator[](Index).GetObject()))
			{
				return true;
			}
		}
		return false;
	}

	// returns true if the subject contains an object matching the given predicate
	template<typename ObjectType = UObject>
	bool Contains(TFunction<bool(const FOverrideStatusObjectHandle<ObjectType>&)> InMatchPredicate) const
	{
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(const FOverrideStatusObjectHandle Handle = GetHandle<ObjectType>(Index))
			{
				if(InMatchPredicate(Handle))
				{
					return true;
				}
			}
		}
		return false;
	}

	// returns the index of a given object or INDEX_NONE
	UE_API int32 Find(const FOverrideStatusObject& InObject) const;

	// returns the index of a given object or INDEX_NONE,
	// the match is determined based on the given predicate 
	template<typename ObjectType = UObject>
	int32 Find(TFunction<bool(const FOverrideStatusObjectHandle<ObjectType>&)> InMatchPredicate) const
	{
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(const FOverrideStatusObjectHandle Handle = GetHandle<ObjectType>(Index))
			{
				if(InMatchPredicate(Handle))
				{
					return true;
				}
			}
		}
		return false;
	}

	// executes a given predicate for each object in the subject
	template<typename ObjectType = UObject>
	void ForEach(TFunction<void(const FOverrideStatusObjectHandle<ObjectType>&)> InPerObjectPredicate) const
	{
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(const FOverrideStatusObjectHandle Handle = GetHandle<ObjectType>(Index))
			{
				InPerObjectPredicate(Handle);
			}
		}
	}

	// returns the first valid value for a given predicate (or an unset TOptional). each object in the subject
	// is interrogated for using the predicate and can return a TOptional indicating success.
	template<typename ValueType, typename ObjectType = UObject>
	TOptional<ValueType> GetFirstValue(TFunction<TOptional<ValueType>(const FOverrideStatusObjectHandle<ObjectType>&)> InGetValuePerObjectPredicate) const
	{
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(const FOverrideStatusObjectHandle Handle = GetHandle<ObjectType>(Index))
			{
				const TOptional<ValueType> SingleValue = InGetValuePerObjectPredicate(Handle);
				if(SingleValue.IsSet())
				{
					return SingleValue;
				}
			}
		}
		return TOptional<ValueType>();
	}

	// returns a common for a given predicate (or an unset TOptional). each object in the subject
	// is interrogated for using the predicate and can return a TOptional indicating success.
	// if the value matches for all objects the value will be returned as a set TOptional, otherwise
	// an unset TOptional will be returned.
	template<typename ValueType, typename ObjectType = UObject>
	TOptional<ValueType> GetCommonValue(TFunction<TOptional<ValueType>(const FOverrideStatusObjectHandle<ObjectType>&)> InGetValuePerObjectPredicate) const
	{
		TOptional<ValueType> Result;
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(const FOverrideStatusObjectHandle Handle = GetHandle<ObjectType>(Index))
			{
				const TOptional<ValueType> SingleValue = InGetValuePerObjectPredicate(Handle);
				if(SingleValue.IsSet())
				{
					if(Result.IsSet())
					{
						if(Result.GetValue() != SingleValue)
						{
							Result.Reset();
							break;
						}
					}
					else
					{
						Result = SingleValue;
					}
				}
			}
		}
		return Result;
	}

	// returns the status provided by the subject's objects or an empty TOptional if it varies across objects
	template<typename ObjectType = UObject>
	TOptional<EOverrideWidgetStatus::Type> GetStatus(TFunction<TOptional<EOverrideWidgetStatus::Type>(const FOverrideStatusObjectHandle<ObjectType>&)> InGetValuePerObjectPredicate) const
	{
		return GetCommonValue<EOverrideWidgetStatus::Type, ObjectType>(InGetValuePerObjectPredicate);
	}

private:
	TArray<FOverrideStatusObject> Objects;
	TSharedPtr<const FPropertyPath> PropertyPath;
	FName SubObjectKey;
	FName Category;
	
	mutable TOptional<FString> LastSeparator;
	mutable TOptional<FString> LastPropertyPathString;
};

#undef UE_API
