// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ToolElementRegistry.h"
#include "ToolMenu.h"
#include "Widgets/SWidget.h"
#include "Overrides/OverrideStatusSubject.h"
#include "Overrides/SOverrideStatusWidget.h"
#include "Overrides/OverrideStatusDetailsWidgetBuilder.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class FPropertyPath;
class FOverrideStatusDetailsDisplayManager;

/**
 * Builder for the override status menu that goes on the top right side of each property
 * Component in the details panel
 */
class FOverrideStatusWidgetMenuBuilder : public FToolElementRegistrationArgs
{

	DECLARE_DELEGATE(FResetToDefault);
	
public:
	/**
	 * constructor
	 *
	 */
	UE_API FOverrideStatusWidgetMenuBuilder(const FOverrideStatusSubject& InSubject, const TWeakPtr<FOverrideStatusDetailsDisplayManager>& InDisplayManager = TWeakPtr<FOverrideStatusDetailsDisplayManager>());

	UE_API virtual ~FOverrideStatusWidgetMenuBuilder() override;

	// returns the status of the override
	UE_API EOverrideWidgetStatus::Type GetStatus() const;

	// returns the attribute backing up the status of the override
	TAttribute<EOverrideWidgetStatus::Type>& GetStatusAttribute() { return StatusAttribute; }

	/**
	 * Override the active overrideable object at the given property path
	 */
	UE_API void AddOverride();
	UE_API bool CanAddOverride() const;
	UE_API FOverrideStatus_AddOverride& OnAddOverride();
	
	/**
	 * Clears any active overrides on the property / object
	 */
	UE_API void ClearOverride();
	UE_API bool CanClearOverride() const;
	UE_API FOverrideStatus_ClearOverride& OnClearOverride();

	/**
	 * Override the active overrideable object at the given property path
	 */
	UE_API void ResetToDefault();
	UE_API bool CanResetToDefault() const;
	UE_API FOverrideStatus_ResetToDefault& OnResetToDefault();
	UE_API FOverrideStatus_ValueDiffersFromDefault& OnValueDiffersFromDefault();

	/**
	 * Set up the menu
	 */
	UE_API void InitializeMenu();

	/**
	 * Fill this method in with your Slate to create it
	 */
	UE_API virtual TSharedPtr<SWidget> GenerateWidget() override;

private:
	/**
	 * The UToolMenu providing the context menu
	 */
	TWeakObjectPtr<UToolMenu> ToolMenu;

	/**
	 * The object that will be queried for its override state
	 */
	FOverrideStatusSubject Subject;

	/**
	 * The status wrapped as a property
	 */
	TAttribute<EOverrideWidgetStatus::Type> StatusAttribute;

	FOverrideStatus_AddOverride AddOverrideDelegate;
	FOverrideStatus_ClearOverride ClearOverrideDelegate;
	FOverrideStatus_ResetToDefault ResetToDefaultDelegate;
	FOverrideStatus_ValueDiffersFromDefault ValueDiffersFromDefaultDelegate;
};

#undef UE_API
