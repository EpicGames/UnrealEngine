//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Overrides/SOverrideStatusWidget.h"
#include "Overrides/OverrideStatusSubject.h"
#include "Widgets/Text/STextBlock.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class SOverrideListWidget : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SOverrideListWidget, SCompoundWidget)
public:
	SLATE_BEGIN_ARGS( SOverrideListWidget )
		: _SubjectsHash(0)
	{}
		SLATE_ATTRIBUTE(uint32, SubjectsHash)
		SLATE_ATTRIBUTE(TArray<FOverrideStatusSubject>, Subjects)
		SLATE_EVENT(FOverrideStatus_GetStatus, OnGetStatus)
        SLATE_EVENT(FOverrideStatus_ClearOverride, OnClearOverride)
	SLATE_END_ARGS()
public:
	UE_API SOverrideListWidget();
	UE_API virtual ~SOverrideListWidget() override;

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
private:

	TAttribute<uint32> SubjectsHashAttribute;
	TAttribute<TArray<FOverrideStatusSubject>> SubjectsAttribute;
	FOverrideStatus_GetStatus GetStatusDelegate;
	FOverrideStatus_ClearOverride ClearOverrideDelegate;

	TSharedPtr<STextBlock> TextBlock;
	TOptional<uint32> LastHash;
};

#undef UE_API
