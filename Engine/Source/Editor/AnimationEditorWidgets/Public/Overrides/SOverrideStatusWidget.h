//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/StyleColors.h"
#include "UObject/OverriddenPropertySet.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class SImage;

namespace EOverrideWidgetStatus
{
	enum Type : int
	{
		None, // No override.
		Undetermined, // State has not yet been determine
		Uninitialized, // This property needs a value.\nYou won't be able to test or publish until you set one. 
		Inherited, // This property's parent component has been overridden.
		ChangedHere, // This property has been overridden.
		ChangedInside, // At least one of this property's values has been overridden.
		ChangedOutside, // A parent property of this property has been overridden.
		Mixed, // The selected elements have mixed override states on this property.
	};
}

struct FOverrideStatusWidgetStyle
{
	const FSlateBrush* Icon = nullptr;
	const FSlateBrush* HoveredIcon = nullptr;
	FText Tooltip;
	FSlateColor Color = FStyleColors::White;
	FSlateColor HoveredColor = FStyleColors::White;
	TOptional<FVector2D> IconSize;
};

DECLARE_DELEGATE_RetVal_OneParam(const FOverrideStatusWidgetStyle*, FOverrideStatusWidgetGetStyle, EOverrideWidgetStatus::Type);
DECLARE_DELEGATE_RetVal_OneParam(FText, FOverrideStatusWidgetGetTooltip, EOverrideWidgetStatus::Type);

class SOverrideStatusWidget : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SOverrideStatusWidget, SCompoundWidget)
public:
	SLATE_BEGIN_ARGS( SOverrideStatusWidget )
	: _Status(EOverrideWidgetStatus::None)
	, _IsHovered(false)
	{}
		SLATE_ATTRIBUTE(EOverrideWidgetStatus::Type, Status)
		SLATE_ARGUMENT(TOptional<FOverrideStatusWidgetStyle>, DefaultStyle)
		SLATE_EVENT(FOverrideStatusWidgetGetStyle, Style)
		SLATE_EVENT(FOverrideStatusWidgetGetTooltip, Tooltip)
		SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_EVENT(FOnGetContent, MenuContent)
		SLATE_ATTRIBUTE(bool, IsHovered)
	SLATE_END_ARGS()
public:
	UE_API SOverrideStatusWidget();
	UE_API virtual ~SOverrideStatusWidget() override;

	UE_API void Construct(const FArguments& InArgs);

	UE_API EOverrideWidgetStatus::Type GetStatus() const;
	UE_API void SetStatus(EOverrideWidgetStatus::Type InStatus);

	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	
	UE_API void QueueRepaint();

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	static UE_API const FOverrideStatusWidgetStyle* GetStyleFromStatus(EOverrideWidgetStatus::Type InStatus);

private:
	
	UE_API FReply HandleClick();
	static UE_API const FOverrideStatusWidgetStyle* GetStyleFromStatus_Fallback(EOverrideWidgetStatus::Type InStatus);
	
	/**
	 * Style used for the button.
	 * Note that it is never null but does change
	 */
	const FOverrideStatusWidgetStyle* WidgetStyle;
	FOverrideStatusWidgetStyle DefaultWidgetStyle;
	TSharedPtr<SImage> Image;
	bool bAlwaysUpdateOnTick;
	bool bTickIsPending;
	TOptional<bool> bLastHovered;
	
	FOnGetContent OnGetMenuContent;
	FOnClicked OnClicked;
	TAttribute<EOverrideWidgetStatus::Type> StatusAttribute;
	FOverrideStatusWidgetGetStyle OnGetStyle;
	FOverrideStatusWidgetGetTooltip OnGetTooltip;
};

#undef UE_API
