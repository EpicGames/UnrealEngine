// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "SchematicGraphModel.h"
#include "SNodePanel.h"
#include "TickableEditorObject.h"
#include "Framework/Animation/AnimatedAttribute.h"

class SSchematicGraphPanel;
class SSchematicGraphNode;

class FSchematicGraphNodeDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSchematicGraphNodeDragDropOp, FDragDropOperation)

	static ANIMATIONEDITORWIDGETS_API TSharedRef<FSchematicGraphNodeDragDropOp> New(TArray<SSchematicGraphNode*> InSchematicGraphNodes, const TArray<FGuid>& InElements);

	ANIMATIONEDITORWIDGETS_API virtual ~FSchematicGraphNodeDragDropOp() override;
	
	ANIMATIONEDITORWIDGETS_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	/** @return true if this drag operation contains property paths */
	bool HasElements() const
	{
		return Elements.Num() > 0;
	}

	/** @return The property paths from this drag operation */
	const TArray<FGuid>& GetElements() const
	{
		return Elements;
	}

	/** @return The nodes being dragged */
	ANIMATIONEDITORWIDGETS_API const TArray<const FSchematicGraphNode*> GetNodes() const;

	ANIMATIONEDITORWIDGETS_API FString GetJoinedDecoratorLabels() const;

private:

	/** Nodes being dragged */
	TArray<SSchematicGraphNode*> SchematicGraphNodes;

	/** Data for the property paths this item represents */
	TArray<FGuid> Elements;
};

class SSchematicGraphNode : public SNodePanel::SNode
{
public:

	typedef TAnimatedAttribute<FVector2d> FVector2dAttribute;
	typedef TAnimatedAttribute<float> FFloatAttribute;
	typedef TAnimatedAttribute<FLinearColor> FLinearColorAttribute;
	
	DECLARE_DELEGATE_TwoParams(FOnClicked, SSchematicGraphNode*, const FPointerEvent&);
	DECLARE_DELEGATE_TwoParams(FOnBeginDrag, SSchematicGraphNode*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_TwoParams(FOnEndDrag, SSchematicGraphNode*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_TwoParams(FOnDrop, SSchematicGraphNode*, const FDragDropEvent&);

	SLATE_BEGIN_ARGS(SSchematicGraphNode);
	SLATE_ARGUMENT(const FSchematicGraphNode*, NodeData)
	SLATE_ARGUMENT(TSharedPtr<FVector2dAttribute>, Position)
	SLATE_ARGUMENT(TSharedPtr<FVector2dAttribute>, Size)
	SLATE_ARGUMENT(TSharedPtr<FFloatAttribute>, Scale)
	SLATE_ATTRIBUTE(bool, EnableAutoScale)
	SLATE_ARGUMENT(TArray<TSharedPtr<FLinearColorAttribute>>, LayerColors)
	SLATE_ARGUMENT(TFunction<const FSlateBrush*(const FGuid&, int32)>, BrushGetter)
	SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_EVENT(FOnBeginDrag, OnBeginDrag)
	SLATE_EVENT(FOnEndDrag, OnEndDrag)
	SLATE_EVENT(FOnDrop, OnDrop)
	SLATE_ATTRIBUTE(FText, ToolTipText)
	SLATE_END_ARGS()

	ANIMATIONEDITORWIDGETS_API void Construct(const FArguments& InArgs);

	ANIMATIONEDITORWIDGETS_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	ANIMATIONEDITORWIDGETS_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	
	ANIMATIONEDITORWIDGETS_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	ANIMATIONEDITORWIDGETS_API virtual EVisibility GetNodeVisibility() const;
	ANIMATIONEDITORWIDGETS_API virtual FVector2f GetPosition2f() const override;
	ANIMATIONEDITORWIDGETS_API void EnablePositionAnimation(bool bEnabled = true);

	const FVector2d& GetOriginalSize() const { return OriginalSize; }

	const FSchematicGraphNode* GetNodeData() const { return NodeData.Get(); }
	FSchematicGraphNode* GetNodeData() { return NodeData.Get(); }
	ANIMATIONEDITORWIDGETS_API const FGuid GetGuid() const;
	ANIMATIONEDITORWIDGETS_API bool IsInteractive() const;

	const bool IsBeingDragged() const { return bIsBeingDragged; }
	ANIMATIONEDITORWIDGETS_API const bool IsFadedOut() const;
	
private:

	bool bIsBeingDragged = false;
	static inline const FVector2d DefaultNodeSize = FVector2d(32.0,32.0);  
	FVector2d OriginalSize = DefaultNodeSize;

	TSharedPtr<FSchematicGraphNode> NodeData;
	TSharedPtr<FVector2dAttribute> Position;
	TOptional<FVector2d> PositionDuringDrag;
	TOptional<FVector2d> OffsetDuringDrag;
	TSharedPtr<FVector2dAttribute> Size;
	TSharedPtr<FFloatAttribute> Scale;
	TAttribute<bool> EnableAutoScale;
	TOptional<float> AutoScale;
	TArray<TSharedPtr<FLinearColorAttribute>> LayerColors;
	TFunction<const FSlateBrush*(const FGuid&, int32)> BrushGetter;
	FOnClicked OnClickedDelegate;
	FOnBeginDrag OnBeginDragDelegate;
	FOnEndDrag OnEndDragDelegate;
	FOnDrop OnDropDelegate;
	SSchematicGraphPanel* SchematicGraphPanel = nullptr;
	TSharedPtr<FFloatAttribute> ExpansionCircleFactor;

	friend class SSchematicGraphPanel;
	friend class FSchematicGraphModel;
};


/** Widget allowing editing of a control rig's structure */
class SSchematicGraphPanel : public SNodePanel, public FTickableEditorObject
{
public:

	using FVector2dAttribute = SSchematicGraphNode::FVector2dAttribute;
	using FFloatAttribute = SSchematicGraphNode::FFloatAttribute;
	using FLinearColorAttribute = SSchematicGraphNode::FLinearColorAttribute;

	DECLARE_DELEGATE_ThreeParams(FOnNodeClicked, SSchematicGraphPanel*, SSchematicGraphNode*, const FPointerEvent&);
	DECLARE_DELEGATE_ThreeParams(FOnBeginDrag, SSchematicGraphPanel*, SSchematicGraphNode*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_ThreeParams(FOnEndDrag, SSchematicGraphPanel*, SSchematicGraphNode*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_TwoParams(FOnEnterDrag, SSchematicGraphPanel*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_TwoParams(FOnLeaveDrag, SSchematicGraphPanel*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_ThreeParams(FOnCancelDrag, SSchematicGraphPanel*, SSchematicGraphNode*, const TSharedPtr<FDragDropOperation>&);
	DECLARE_DELEGATE_ThreeParams(FOnDrop, SSchematicGraphPanel*, SSchematicGraphNode*, const FDragDropEvent&);
	
	SLATE_BEGIN_ARGS(SSchematicGraphPanel)
	{}
	SLATE_ARGUMENT(bool, IsOverlay)
	SLATE_ARGUMENT_DEPRECATED(FSchematicGraphModel*, GraphData, 5.6, "Please, use GraphDataModel instead.")
	SLATE_ARGUMENT(TWeakPtr<FSchematicGraphModel>, GraphDataModel)
	SLATE_ARGUMENT(int32, PaddingLeft)
	SLATE_ARGUMENT(int32, PaddingRight)
	SLATE_ARGUMENT(int32, PaddingTop)
	SLATE_ARGUMENT(int32, PaddingBottom)
	SLATE_ARGUMENT(int32, PaddingInterNode)
	SLATE_EVENT(FOnNodeClicked, OnNodeClicked)
	SLATE_EVENT(FOnBeginDrag, OnBeginDrag)
	SLATE_EVENT(FOnEndDrag, OnEndDrag)
	SLATE_EVENT(FOnEnterDrag, OnEnterDrag)
	SLATE_EVENT(FOnLeaveDrag, OnLeaveDrag)
	SLATE_EVENT(FOnCancelDrag, OnCancelDrag)
	SLATE_EVENT(FOnDrop, OnDrop)
	SLATE_END_ARGS()

	struct FSchematicLinkWidgetInfo
	{
		TSharedPtr<FFloatAttribute> Minimum;
		TSharedPtr<FFloatAttribute> Maximum;
		TSharedPtr<FLinearColorAttribute> Color;
		TSharedPtr<FFloatAttribute> Thickness;
	};

	virtual ~SSchematicGraphPanel() override
	{
		OnNodeClickedDelegate.Unbind();
		OnDropDelegate.Unbind();
	}

	UE_DEPRECATED(5.6, "SetSchematicGraph with raw pointer has been deprecated, please, use SetSchematicGraphModel with a weak pointer instead.")
	void SetSchematicGraph(FSchematicGraphModel* InGraphData) {}
	UE_DEPRECATED(5.6, "GetSchematicGraph with raw pointer has been deprecated, please, use GetSchematicGraphModel with a weak pointer instead.")
	const FSchematicGraphModel* GetSchematicGraph() const { return nullptr; }
	UE_DEPRECATED(5.6, "GetSchematicGraph with raw pointer has been deprecated, please, use GetSchematicGraphModel with a weak pointer instead.")
	FSchematicGraphModel* GetSchematicGraph() { return nullptr; }

	ANIMATIONEDITORWIDGETS_API void SetSchematicGraphModel(TWeakPtr<FSchematicGraphModel> InGraphData);
	const TWeakPtr<const FSchematicGraphModel> GetSchematicGraphModel() const { return GraphDataWeak; }
	TWeakPtr<FSchematicGraphModel> GetSchematicGraphModel() { return GraphDataWeak; }
	ANIMATIONEDITORWIDGETS_API void Construct(const FArguments& InArgs);

	ANIMATIONEDITORWIDGETS_API void RebuildPanel();
	ANIMATIONEDITORWIDGETS_API void AddNode(const FSchematicGraphNode* InNodeToAdd);
	ANIMATIONEDITORWIDGETS_API void RemoveNode(const FSchematicGraphNode* InNodeToRemove);
	ANIMATIONEDITORWIDGETS_API const SSchematicGraphNode* FindNode(const FGuid& InGuid) const;
	ANIMATIONEDITORWIDGETS_API SSchematicGraphNode* FindNode(const FGuid& InGuid);
	ANIMATIONEDITORWIDGETS_API void AddLink(const FSchematicGraphLink* InLinkToAdd);
	ANIMATIONEDITORWIDGETS_API void RemoveLink(const FSchematicGraphLink* InLinkToRemove);
	ANIMATIONEDITORWIDGETS_API const FSchematicLinkWidgetInfo* FindLink(const FGuid& InGuid) const;
	ANIMATIONEDITORWIDGETS_API FSchematicLinkWidgetInfo* FindLink(const FGuid& InGuid);

	// SNodePanel interface
	ANIMATIONEDITORWIDGETS_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	ANIMATIONEDITORWIDGETS_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	ANIMATIONEDITORWIDGETS_API virtual void RemoveAllNodes() override;
	ANIMATIONEDITORWIDGETS_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	// End of SNodePanel interface
	
	ANIMATIONEDITORWIDGETS_API TSharedRef<SSchematicGraphNode> GetChild(int32 ChildIndex) const;

	// FTickableEditorObject Interface
	ANIMATIONEDITORWIDGETS_API virtual TStatId GetStatId() const override;
	ANIMATIONEDITORWIDGETS_API virtual void Tick(float DeltaTime) override;
	ANIMATIONEDITORWIDGETS_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual bool IsTickable() const override { return true; }
	// End of FTickableEditorObject interface

	ANIMATIONEDITORWIDGETS_API void ToggleVisibility();
	ANIMATIONEDITORWIDGETS_API void OnNodeClicked(SSchematicGraphNode* Node, const FPointerEvent& MouseEvent);
	ANIMATIONEDITORWIDGETS_API void OnBeginDragEvent(SSchematicGraphNode* Node, const TSharedPtr<FDragDropOperation>& InDragDropEvent);
	ANIMATIONEDITORWIDGETS_API void OnEndDragEvent(SSchematicGraphNode* Node, const TSharedPtr<FDragDropOperation>& InDragDropEvent);
	ANIMATIONEDITORWIDGETS_API void OnEnterDragEvent(const TSharedPtr<FDragDropOperation>& InDragDropEvent);
	ANIMATIONEDITORWIDGETS_API void OnLeaveDragEvent(const TSharedPtr<FDragDropOperation>& InDragDropEvent);
	ANIMATIONEDITORWIDGETS_API void OnCancelDragEvent(SSchematicGraphNode* Node, const TSharedPtr<FDragDropOperation>& InDragDropEvent);
	ANIMATIONEDITORWIDGETS_API void OnDropEvent(SSchematicGraphNode* Node, const FDragDropEvent& InDragDropEvent);
	ANIMATIONEDITORWIDGETS_API FReply HandleNodeDragDetected(FGuid Guid, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	ANIMATIONEDITORWIDGETS_API virtual FVector2d GetPositionForNode(FGuid InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FLinearColor GetColorForNode(FGuid InNodeGuid, int32 InLayerIndex) const;
	ANIMATIONEDITORWIDGETS_API virtual FText GetToolTipForNode(FGuid InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetScaleForNode(FGuid InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API void AdjustPositionWithDPIScale(FVector2d& InOutPosition, bool bInverse = false) const;

	ANIMATIONEDITORWIDGETS_API virtual bool IsAutoGroupingEnabled() const;
	ANIMATIONEDITORWIDGETS_API virtual float GetAutoGroupingDistance() const;

	ANIMATIONEDITORWIDGETS_API virtual bool IsAutoScaleEnabledForNode(FGuid InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetMinimumLinkDistanceForNode(FGuid InLinkGuid, bool bIncludeScale = true) const;

	FOnNodeClicked& OnNodeClicked() { return OnNodeClickedDelegate; }
	FOnBeginDrag& OnBeginDrag() { return OnBeginDragDelegate; }
	FOnEndDrag& OnEndDrag() { return OnEndDragDelegate; }
	FOnEnterDrag& OnEnterDrag() { return OnEnterDragDelegate; }
	FOnLeaveDrag& OnLeaveDrag() { return OnLeaveDragDelegate; }
	FOnCancelDrag& OnCancelDrag() { return OnCancelDragDelegate; }
	FOnDrop& OnAcceptDrop() { return OnDropDelegate; }

	const FVector2d& GetNodeLabelOffset() const { return NodeLabelOffset; }
	ANIMATIONEDITORWIDGETS_API void IncrementNodeLabelOffset(const FVector2d& InOffset);

private:

	ANIMATIONEDITORWIDGETS_API void UpdatePerNodeCaches(bool bRemoveNodesFromAutoGroups);
	ANIMATIONEDITORWIDGETS_API void UpdateAutoGroupingForNodes();
	ANIMATIONEDITORWIDGETS_API void UpdateAutoScalingForNodes();

	bool bIsDragDropping = false;
	TSharedPtr<FDragDropOperation> DragDropOpFromOutside;
	bool bIsOverlay = false;
	int32 PaddingLeft = 0;
	int32 PaddingRight = 0;
	int32 PaddingTop = 0;
	int32 PaddingBottom = 0;
	int32 PaddingInterNode = 0;
	TWeakPtr<FSchematicGraphModel> GraphDataWeak;
	FOnNodeClicked OnNodeClickedDelegate;
	FOnBeginDrag OnBeginDragDelegate;
	FOnEndDrag OnEndDragDelegate;
	FOnEnterDrag OnEnterDragDelegate;
	FOnLeaveDrag OnLeaveDragDelegate;
	FOnCancelDrag OnCancelDragDelegate;
	FOnDrop OnDropDelegate;
	TMap<FGuid, TSharedPtr<SSchematicGraphNode>> NodeByGuid;

	struct FPerNodeCache
	{
		FPerNodeCache()
			: Guid()
			, Label()
			, bHasParent(false)
			, Visibility(ESchematicGraphVisibility::Visible)
			, bIsAutoScaling(false)
			, Position(FVector2d::ZeroVector)
			, Radius(0.0)
		{}

		FGuid Guid;
		FText Label;
		bool bHasParent;
		ESchematicGraphVisibility::Type Visibility;
		bool bIsAutoScaling;
		FVector2d Position;
		double Radius;
	};

	TArray<FPerNodeCache> PerNodeCaches;
	TMap<FGuid, int32> GuidToNodeCache;
	TMap<uint32, FGuid> GroupNodeGuidByHash;

	TMap<FGuid, TSharedPtr<FSchematicLinkWidgetInfo>> LinkByGuid;

	mutable TMap<FGuid, FVector2d> NodeCenterByGuid;
	mutable TArray<FVector2d> NodeCenterByIndex;
	mutable TArray<ESchematicGraphVisibility::Type> NodeVisibilityByIndex;
	mutable TMap<FGuid, ESchematicGraphVisibility::Type> NodeVisibilityByGuid;
	mutable TOptional<FGuid> DropTarget;
	mutable TOptional<float> DPIScale;
	FVector2d NodeLabelOffset = FVector2d::ZeroVector;
};

#endif
