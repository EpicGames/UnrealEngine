// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"

ANIMATIONEDITORWIDGETS_API DECLARE_LOG_CATEGORY_EXTERN(LogSchematicGraph, Log, All);

class FSchematicGraphTag;
class FSchematicGraphNode;
class FSchematicGraphLink;
class FSchematicGraphModel;

namespace ESchematicGraphVisibility
{
	enum Type : int
	{
		Visible,
		FadedOut,
		Hidden
	};
}

DECLARE_EVENT_TwoParams(FSchematicGraph, FOnSchematicGraphTagAdded, const FSchematicGraphNode*, const FSchematicGraphTag*);
DECLARE_EVENT_TwoParams(FSchematicGraph, FOnTagRemoved, const FSchematicGraphNode*, const FSchematicGraphTag*);
DECLARE_EVENT_OneParam(FSchematicGraph, FOnSchematicGraphNodeAdded, const FSchematicGraphNode*);
DECLARE_EVENT_OneParam(FSchematicGraph, FOnNodeRemoved, const FSchematicGraphNode*);
DECLARE_EVENT_OneParam(FSchematicGraph, FOnSchematicGraphLinkAdded, const FSchematicGraphLink*);
DECLARE_EVENT_OneParam(FSchematicGraph, FOnLinkRemoved, const FSchematicGraphLink*);
DECLARE_EVENT(FSchematicGraph, FOnGraphReset);

#define SCHEMATICGRAPHELEMENT_BODY_BASE(ClassName) \
virtual ~ClassName() {} \
inline static const FName& Type = TEXT(#ClassName); \
virtual const FName& GetType() const \
{ \
	return ClassName::Type;  \
} \
virtual bool IsA(const FName& InType) const \
{ \
	return Type == InType; \
} \
template<typename ElementType> \
bool IsA() const \
{ \
	return IsA(ElementType::Type); \
} \
\
template<typename T> \
friend const T* Cast(const ClassName* InElement) \
{ \
	if(InElement) \
	{ \
		if(InElement->IsA<T>()) \
		{ \
			return static_cast<const T*>(InElement); \
		} \
	} \
	return nullptr; \
} \
\
template<typename T> \
friend T* Cast(ClassName* InElement) \
{ \
	if(InElement) \
	{ \
		if(InElement->IsA<T>()) \
		{ \
			return static_cast<T*>(InElement); \
		} \
	} \
	return nullptr; \
} \
\
template<typename T> \
friend const T* CastChecked(const ClassName* InElement) \
{ \
	const T* Element = Cast<T>(InElement); \
	check(Element); \
	return Element; \
} \
\
template<typename T> \
friend T* CastChecked(ClassName* InElement) \
{ \
	T* Element = Cast<T>(InElement); \
	check(Element); \
	return Element; \
}

#define SCHEMATICGRAPHELEMENT_BODY(ClassName, SuperClass, BaseClass) \
typedef SuperClass Super; \
inline static const FName& Type = TEXT(#ClassName); \
virtual const FName& GetType() const override { return ClassName::Type; } \
virtual bool IsA(const FName& InType) const \
{ \
	if(ClassName::Type == InType) \
	{ \
		return true; \
	} \
	return SuperClass::IsA(InType); \
} \
template<typename T> \
friend const T* Cast(const ClassName* InElement) \
{ \
	return Cast<T>((const BaseClass*) InElement); \
} \
template<typename T> \
friend T* Cast(ClassName* InElement) \
{ \
	return Cast<T>((BaseClass*) InElement); \
} \
template<typename T> \
friend const T* CastChecked(const ClassName* InElement) \
{ \
	return CastChecked<T>((const BaseClass*) InElement); \
} \
template<typename T> \
friend T* CastChecked(ClassName* InElement) \
{ \
	return CastChecked<T>((BaseClass*) InElement); \
}

#endif