// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "SchematicGraphDefines.h"

#define SCHEMATICGRAPHLINK_BODY(ClassName, SuperClass) \
SCHEMATICGRAPHELEMENT_BODY(ClassName, SuperClass, FSchematicGraphNode)

class FSchematicGraphLink : public TSharedFromThis<FSchematicGraphLink>
{
public:

	SCHEMATICGRAPHELEMENT_BODY_BASE(FSchematicGraphLink)
	
	const FGuid& GetGuid() const { return Guid; }
	static uint32 GetLinkHash(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid)
	{
		return HashCombine(GetTypeHash(InSourceNodeGuid), GetTypeHash(InTargetNodeGuid));
	}
	uint32 GetLinkHash() const { return GetLinkHash(SourceNodeGuid, TargetNodeGuid); }
	const FGuid& GetSourceNodeGuid() const { return SourceNodeGuid; }
	const FGuid& GetTargetNodeGuid() const { return TargetNodeGuid; }
	virtual float GetMinimum() const { return Minimum; }
	virtual float GetMaximum() const { return Maximum; }
	virtual FVector2d GetSourceNodeOffset() const { return SourceNodeOffset; }
	virtual FVector2d GetTargetNodeOffset() const { return TargetNodeOffset; }
	virtual FLinearColor GetColor() const { return Color; }
	virtual float GetThickness() const { return Thickness; }
	virtual const FSlateBrush* GetBrush() const { return Brush; }
	virtual const FText& GetToolTip() const { return ToolTip; }
	virtual ESchematicGraphVisibility::Type GetVisibility() const { return Visibility; }
	virtual void SetVisibility(ESchematicGraphVisibility::Type InVisibility) { Visibility = InVisibility; }

protected:
	
	FSchematicGraphModel* Model = nullptr;
	FGuid Guid = FGuid::NewGuid();
	FGuid SourceNodeGuid = FGuid();
	FGuid TargetNodeGuid = FGuid();
	float Minimum = 0.f;
	float Maximum = 1.f;
	FVector2d SourceNodeOffset = FVector2d::ZeroVector;
	FVector2d TargetNodeOffset = FVector2d::ZeroVector;
	FLinearColor Color = FLinearColor::White;
	float Thickness = 1.f;
	const FSlateBrush* Brush = nullptr;
	FText ToolTip = FText();
	ESchematicGraphVisibility::Type Visibility = ESchematicGraphVisibility::Visible;

	friend class FSchematicGraphModel;
};

#endif
