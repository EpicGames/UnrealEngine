// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "SchematicGraphTag.h"
#include "SchematicGraphNode.h"
#include "SchematicGraphLink.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

class SSchematicGraphPanel;

class FSchematicGraphModel : public TSharedFromThis<FSchematicGraphModel>
{
public:

	SCHEMATICGRAPHELEMENT_BODY_BASE(FSchematicGraphModel)
	
	ANIMATIONEDITORWIDGETS_API virtual void Reset();
	virtual void ApplyToPanel(SSchematicGraphPanel* InPanel) {};
	
	template<typename NodeType = FSchematicGraphNode>
	NodeType* AddNode(bool bNotify = true)
	{
		const TSharedPtr<FSchematicGraphNode> NewNode = MakeShareable(new NodeType);
		NewNode->Model = this;
		Nodes.Add(NewNode);
		NodeByGuid.Add(NewNode->GetGuid(), NewNode);

		if (bNotify && OnNodeAddedDelegate.IsBound())
		{
			OnNodeAddedDelegate.Broadcast(NewNode.Get());
		}
		return static_cast<NodeType*>(NewNode.Get());
	}

	template<typename NodeType = FSchematicGraphNode>
	const NodeType* FindNode(const FGuid& InNodeGuid) const
	{
		if(const TSharedPtr<FSchematicGraphNode>* ExistingNode = NodeByGuid.Find(InNodeGuid))
		{
			return Cast<NodeType>(ExistingNode->Get());
		};
		return nullptr;
	}

	template<typename NodeType = FSchematicGraphNode>
	NodeType* FindNode(const FGuid& InNodeGuid)
	{
		const FSchematicGraphModel* ConstThis = this;
		return const_cast<NodeType*>(ConstThis->FindNode<NodeType>(InNodeGuid));
	}

	template<typename NodeType = FSchematicGraphNode>
	const NodeType* FindNodeChecked(const FGuid& InNodeGuid) const
	{
		if(const TSharedPtr<FSchematicGraphNode>* ExistingNode = NodeByGuid.Find(InNodeGuid))
		{
			check(ExistingNode->Get()->IsA(NodeType::Type));
			return static_cast<NodeType*>(ExistingNode->Get());
		};
		return nullptr;
	}

	template<typename NodeType = FSchematicGraphNode>
	NodeType* FindNodeChecked(const FGuid& InNodeGuid)
	{
		const FSchematicGraphModel* ConstThis = this;
		return const_cast<NodeType*>(ConstThis->FindNodeChecked<NodeType>(InNodeGuid));
	}

	ANIMATIONEDITORWIDGETS_API virtual bool RemoveNode(const FGuid& InNodeGuid);

	ANIMATIONEDITORWIDGETS_API virtual bool SetParentNode(const FGuid& InChildNodeGuid, const FGuid& InParentNodeGuid, bool bUpdateGroupNode = true);
	ANIMATIONEDITORWIDGETS_API virtual bool SetParentNode(const FSchematicGraphNode* InChildNode, const FSchematicGraphNode* InParentNode, bool bUpdateGroupNode = true);
	ANIMATIONEDITORWIDGETS_API bool RemoveFromParentNode(const FGuid& InChildNodeGuid, bool bUpdateGroupNode = true);
	ANIMATIONEDITORWIDGETS_API bool RemoveFromParentNode(const FSchematicGraphNode* InChildNode, bool bUpdateGroupNode = true);

	const TArray<TSharedPtr<FSchematicGraphNode>>& GetNodes() const { return Nodes; }

	ANIMATIONEDITORWIDGETS_API FVector2d GetPositionForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FVector2d GetPositionForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API FVector2d GetPositionOffsetForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FVector2d GetPositionOffsetForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API bool GetPositionAnimationEnabledForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual bool GetPositionAnimationEnabledForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API FVector2d GetSizeForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FVector2d GetSizeForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API float GetScaleForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetScaleForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API float GetScaleOffsetForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetScaleOffsetForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API float GetMinimumLinkDistanceForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetMinimumLinkDistanceForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API bool IsAutoScaleEnabledForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual bool IsAutoScaleEnabledForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API int32 GetNumLayersForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual int32 GetNumLayersForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API FLinearColor GetColorForNode(const FGuid& InNodeGuid, int32 InLayerIndex) const;
	ANIMATIONEDITORWIDGETS_API virtual FLinearColor GetColorForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const;
	ANIMATIONEDITORWIDGETS_API const FSlateBrush* GetBrushForNode(const FGuid& InNodeGuid, int32 InLayerIndex) const;
	ANIMATIONEDITORWIDGETS_API virtual const FSlateBrush* GetBrushForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const;
	ANIMATIONEDITORWIDGETS_API FText GetToolTipForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FText GetToolTipForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API ESchematicGraphVisibility::Type GetVisibilityForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual ESchematicGraphVisibility::Type GetVisibilityForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API TOptional<ESchematicGraphVisibility::Type> GetVisibilityForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<ESchematicGraphVisibility::Type> GetVisibilityForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const;
	ANIMATIONEDITORWIDGETS_API TOptional<FVector2d> GetPositionForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<FVector2d> GetPositionForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const;
	ANIMATIONEDITORWIDGETS_API TOptional<float> GetScaleForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<float> GetScaleForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const;
	ANIMATIONEDITORWIDGETS_API TOptional<bool> GetInteractivityForChildNode(const FGuid& InParentNodeGuid, const FGuid& InChildNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<bool> GetInteractivityForChildNode(const FSchematicGraphNode* InParentNode, const FSchematicGraphNode* InChildNode) const;
	ANIMATIONEDITORWIDGETS_API bool IsDragSupportedForNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual bool IsDragSupportedForNode(const FSchematicGraphNode* InNode) const;
	ANIMATIONEDITORWIDGETS_API virtual bool GetContextMenuForNode(const FSchematicGraphNode* InNode, FMenuBuilder& OutMenu) const;
	ANIMATIONEDITORWIDGETS_API const TArray<TSharedPtr<FSchematicGraphNode>> GetSelectedNodes() const;
	ANIMATIONEDITORWIDGETS_API void ClearSelection();

	ANIMATIONEDITORWIDGETS_API FLinearColor GetBackgroundColorForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FLinearColor GetBackgroundColorForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API FLinearColor GetForegroundColorForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FLinearColor GetForegroundColorForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API FLinearColor GetLabelColorForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FLinearColor GetLabelColorForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API const FSlateBrush* GetBackgroundBrushForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual const FSlateBrush* GetBackgroundBrushForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API const FSlateBrush* GetForegroundBrushForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual const FSlateBrush* GetForegroundBrushForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API const FText GetLabelForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual const FText GetLabelForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API const FText GetToolTipForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual const FText GetToolTipForTag(const FSchematicGraphTag* InTag) const;
	ANIMATIONEDITORWIDGETS_API ESchematicGraphVisibility::Type GetVisibilityForTag(const FGuid& InNodeGuid, const FGuid& InTagGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual ESchematicGraphVisibility::Type GetVisibilityForTag(const FSchematicGraphTag* InTag) const;

	template<typename LinkType = FSchematicGraphLink>
	LinkType* AddLink(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid, bool bNotify = true)
	{
		const TSharedPtr<FSchematicGraphLink> NewLink = MakeShareable(new LinkType);
		NewLink->Model = this;
		NewLink->SourceNodeGuid = InSourceNodeGuid;
		NewLink->TargetNodeGuid = InTargetNodeGuid;
		Links.Add(NewLink);
		LinkByGuid.Add(NewLink->GetGuid(), NewLink);
		LinkByHash.Add(NewLink->GetLinkHash(), NewLink);
		NodeGuidToLinkGuids.FindOrAdd(InSourceNodeGuid).Get<0>().Add(NewLink->GetGuid());
		NodeGuidToLinkGuids.FindOrAdd(InTargetNodeGuid).Get<1>().Add(NewLink->GetGuid());

		if (bNotify && OnLinkAddedDelegate.IsBound())
		{
			OnLinkAddedDelegate.Broadcast(NewLink.Get());
		}
		return static_cast<LinkType*>(NewLink.Get());
	}

	template<typename LinkType = FSchematicGraphLink>
	const LinkType* FindLink(const FGuid& InLinkGuid) const
	{
		if(const TSharedPtr<FSchematicGraphLink>* ExistingLink = LinkByGuid.Find(InLinkGuid))
		{
			return Cast<LinkType>(ExistingLink->Get());
		};
		return nullptr;
	}

	template<typename LinkType = FSchematicGraphLink>
	LinkType* FindLink(const FGuid& InLinkGuid)
	{
		const FSchematicGraphModel* ConstThis = this;
		return const_cast<LinkType*>(ConstThis->FindLink<LinkType>(InLinkGuid));
	}

	template<typename LinkType = FSchematicGraphLink>
	const LinkType* FindLinkChecked(const FGuid& InLinkGuid) const
	{
		if(const TSharedPtr<FSchematicGraphLink>* ExistingLink = LinkByGuid.Find(InLinkGuid))
		{
			check(ExistingLink->Get()->IsA(LinkType::Type));
			return static_cast<LinkType*>(ExistingLink->Get());
		};
		return nullptr;
	}

	template<typename LinkType = FSchematicGraphLink>
	LinkType* FindLinkChecked(const FGuid& InLinkGuid)
	{
		const FSchematicGraphModel* ConstThis = this;
		return const_cast<LinkType*>(ConstThis->FindLinkChecked<LinkType>(InLinkGuid));
	}
	
	template<typename LinkType = FSchematicGraphLink>
	const LinkType* FindLink(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid) const
	{
		const uint32 LinkHash = FSchematicGraphLink::GetLinkHash(InSourceNodeGuid, InTargetNodeGuid);
		if(const TSharedPtr<FSchematicGraphLink>* ExistingLink = LinkByHash.Find(LinkHash))
		{
			return Cast<LinkType>(ExistingLink->Get());
		};
		return nullptr;
	}

	template<typename LinkType = FSchematicGraphLink>
	LinkType* FindLink(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid)
	{
		const FSchematicGraphModel* ConstThis = this;
		return const_cast<LinkType*>(ConstThis->FindLink<LinkType>(InSourceNodeGuid, InTargetNodeGuid));
	}

	template<typename LinkType = FSchematicGraphLink>
	const LinkType* FindLinkChecked(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid) const
	{
		const uint32 LinkHash = FSchematicGraphLink::GetLinkHash(InSourceNodeGuid, InTargetNodeGuid);
		if(const TSharedPtr<FSchematicGraphLink>* ExistingLink = LinkByHash.Find(LinkHash))
		{
			check(ExistingLink->Get()->IsA(LinkType::Type));
			return Cast<LinkType>(ExistingLink->Get());
		};
		return nullptr;
	}

	template<typename LinkType = FSchematicGraphLink>
	LinkType* FindLinkChecked(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid)
	{
		const FSchematicGraphModel* ConstThis = this;
		return const_cast<LinkType*>(ConstThis->FindLinkChecked<LinkType>(InSourceNodeGuid, InTargetNodeGuid));
	}

	ANIMATIONEDITORWIDGETS_API bool IsLinkedTo(const FGuid& InSourceNodeGuid, const FGuid& InTargetNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API TArray<const FSchematicGraphLink*> FindLinksOnNode(const FGuid& InNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API TArray<const FSchematicGraphLink*> FindLinksOnSource(const FGuid& InSourceNodeGuid) const;
	ANIMATIONEDITORWIDGETS_API TArray<const FSchematicGraphLink*> FindLinksOnTarget(const FGuid& InTargetNodeGuid) const;

	ANIMATIONEDITORWIDGETS_API virtual bool RemoveLink(const FGuid& InLinkGuid);

	const TArray<TSharedPtr<FSchematicGraphLink>>& GetLinks() const { return Links; }

	ANIMATIONEDITORWIDGETS_API float GetMinimumForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetMinimumForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API float GetMaximumForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetMaximumForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API FVector2d GetSourceNodeOffsetForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API FVector2d GetSourceNodeOffsetForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API FVector2d GetTargetNodeOffsetForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API FVector2d GetTargetNodeOffsetForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API FLinearColor GetColorForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual FLinearColor GetColorForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API float GetThicknessForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual float GetThicknessForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API const FSlateBrush* GetBrushForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual const FSlateBrush* GetBrushForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API const FText GetToolTipForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual const FText GetToolTipForLink(const FSchematicGraphLink* InLink) const;
	ANIMATIONEDITORWIDGETS_API ESchematicGraphVisibility::Type GetVisibilityForLink(const FGuid& InLinkGuid) const;
	ANIMATIONEDITORWIDGETS_API virtual ESchematicGraphVisibility::Type GetVisibilityForLink(const FSchematicGraphLink* InLink) const;

	virtual bool IsAutoGroupingEnabled() const { return true; }
	virtual float GetAutoGroupingDistance() const { return 4.f; }
	ANIMATIONEDITORWIDGETS_API virtual FSchematicGraphGroupNode* AddAutoGroupNode(); 

	FOnSchematicGraphNodeAdded& OnNodeAdded() { return OnNodeAddedDelegate; }
	FOnNodeRemoved& OnNodeRemoved() { return OnNodeRemovedDelegate; }
	FOnSchematicGraphLinkAdded& OnLinkAdded() { return OnLinkAddedDelegate; }
	FOnLinkRemoved& OnLinkRemoved() { return OnLinkRemovedDelegate; }
	FOnSchematicGraphTagAdded& OnTagAdded() { return OnTagAddedDelegate; }
	FOnTagRemoved& OnTagRemoved() { return OnTagRemovedDelegate; }
	FOnGraphReset& OnGraphReset() { return OnGraphResetDelegate; }

	virtual bool GetForwardedNodeForDrag(FGuid& InOutGuid) const { return false; }

	ANIMATIONEDITORWIDGETS_API const FSchematicGraphGroupNode* GetLastExpandedNode() const;
	ANIMATIONEDITORWIDGETS_API virtual void SetLastExpandedNode(const FSchematicGraphGroupNode* InGroupNode);
	void ClearLastExpandedNode()
	{
		SetLastExpandedNode(nullptr);
	}

	ANIMATIONEDITORWIDGETS_API virtual void Tick(float InDeltaTime);

protected:

	TArray<TSharedPtr<FSchematicGraphNode>> Nodes;
	TMap<FGuid, TSharedPtr<FSchematicGraphNode>> NodeByGuid;
	TArray<TSharedPtr<FSchematicGraphLink>> Links;
	TMap<FGuid, TSharedPtr<FSchematicGraphLink>> LinkByGuid;
	TMap<uint32, TSharedPtr<FSchematicGraphLink>> LinkByHash;
	TMap<FGuid, TTuple<TArray<FGuid>, TArray<FGuid>>> NodeGuidToLinkGuids;
	FGuid LastExpandedNode = FGuid();

	FOnSchematicGraphNodeAdded OnNodeAddedDelegate;
	FOnNodeRemoved OnNodeRemovedDelegate;
	FOnSchematicGraphLinkAdded OnLinkAddedDelegate;
	FOnLinkRemoved OnLinkRemovedDelegate;
	FOnSchematicGraphTagAdded OnTagAddedDelegate;
	FOnTagRemoved OnTagRemovedDelegate;
	FOnGraphReset OnGraphResetDelegate;

	friend class FSchematicGraphNode;
};

#endif
