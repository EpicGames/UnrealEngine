// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "Framework/Animation/AnimatedAttribute.h"
#include "SchematicGraphTag.h"
#include "Input/Events.h"
#include "Input/Reply.h"

#define SCHEMATICGRAPHNODE_BODY(ClassName, SuperClass) \
SCHEMATICGRAPHELEMENT_BODY(ClassName, SuperClass, FSchematicGraphNode)

class FSchematicGraphGroupNode;

class FSchematicGraphNode : public TSharedFromThis<FSchematicGraphNode>
{
public:

	SCHEMATICGRAPHELEMENT_BODY_BASE(FSchematicGraphNode)

	ANIMATIONEDITORWIDGETS_API FSchematicGraphNode();
	
	const FGuid& GetGuid() const { return Guid; }
	const FSchematicGraphModel* GetGraph() const { return Model; }
	bool HasParentNode() const { return ParentNodeGuid.IsValid(); }
	const FGuid& GetParentNodeGuid() const { return ParentNodeGuid; }
	ANIMATIONEDITORWIDGETS_API const FSchematicGraphNode* GetParentNode() const;
	ANIMATIONEDITORWIDGETS_API FSchematicGraphNode* GetParentNode();
	ANIMATIONEDITORWIDGETS_API const FGuid& GetRootNodeGuid() const;
	ANIMATIONEDITORWIDGETS_API const FSchematicGraphNode* GetRootNode() const;
	ANIMATIONEDITORWIDGETS_API FSchematicGraphNode* GetRootNode();
	bool IsRootNode() const { return !HasParentNode(); }
	ANIMATIONEDITORWIDGETS_API FSchematicGraphGroupNode* GetGroupNode();
	ANIMATIONEDITORWIDGETS_API const FSchematicGraphGroupNode* GetGroupNode() const;
	int32 GetNumChildNodes() const { return ChildNodeGuids.Num(); }
	const TArray<FGuid>& GetChildNodeGuids() const { return ChildNodeGuids; }
	ANIMATIONEDITORWIDGETS_API const FSchematicGraphNode* GetChildNode(int32 InChildNodeIndex) const;
	ANIMATIONEDITORWIDGETS_API FSchematicGraphNode* GetChildNode(int32 InChildNodeIndex);
	ANIMATIONEDITORWIDGETS_API TOptional<ESchematicGraphVisibility::Type> GetVisibilityForChildNode(const FGuid& InChildGuid) const;
	virtual TOptional<ESchematicGraphVisibility::Type> GetVisibilityForChildNode(const FSchematicGraphNode* InChildNode) const { return  TOptional<ESchematicGraphVisibility::Type>(); }
	ANIMATIONEDITORWIDGETS_API TOptional<FVector2d> GetPositionForChildNode(const FGuid& InChildGuid) const;
	virtual TOptional<FVector2d> GetPositionForChildNode(const FSchematicGraphNode* InChildNode) const { return TOptional<FVector2d>(); }
	ANIMATIONEDITORWIDGETS_API TOptional<float> GetScaleForChildNode(const FGuid& InChildGuid) const;
	virtual TOptional<float> GetScaleForChildNode(const FSchematicGraphNode* InChildNode) const { return TOptional<float>(); }
	ANIMATIONEDITORWIDGETS_API TOptional<bool> GetInteractivityForChildNode(const FGuid& InChildGuid) const;
	virtual TOptional<bool> GetInteractivityForChildNode(const FSchematicGraphNode* InChildNode) const { return TOptional<bool>(); }
	bool IsSelected() const { return bIsSelected; }
	void SetSelected(bool bSelected = true) { bIsSelected = bSelected; }
	virtual const FText& GetLabel() const { return Label; }
	void SetLabel(const FText& InLabel) { Label = InLabel; }
	ANIMATIONEDITORWIDGETS_API virtual FVector2d GetPosition() const;
	virtual void SetPosition(const FVector2d& InPosition) { Position = InPosition; }
	virtual FVector2d GetPositionOffset() const { return PositionOffset; }
	virtual void SetPositionOffset(const FVector2d& InPositionOffset) { PositionOffset = InPositionOffset; }
	virtual float GetScaleOffset() const { return ScaleOffset; }
	virtual void SetScaleOffset(float InScaleOffset) { ScaleOffset = InScaleOffset; }
	int32 GetNumLayers() const { return FMath::Min(Colors.Num(), Brushes.Num()); }
	virtual FLinearColor GetColor(int32 InLayerIndex) const { return Colors[InLayerIndex]; }
	virtual const FSlateBrush* GetBrush(int32 InLayerIndex) const { return Brushes[InLayerIndex]; }
	virtual bool IsAutoScaleEnabled() const { return false; }
	ANIMATIONEDITORWIDGETS_API virtual const FText& GetToolTip() const;
	ANIMATIONEDITORWIDGETS_API virtual ESchematicGraphVisibility::Type GetVisibility() const;
	virtual void SetVisibility(ESchematicGraphVisibility::Type InVisibility) { Visibility = InVisibility; }
	ANIMATIONEDITORWIDGETS_API virtual bool IsInteractive() const;
	virtual bool IsDragSupported() const { return bDragSupported; }
	virtual void SetDragSupported(bool InDragSupported) { bDragSupported = InDragSupported;}

	ANIMATIONEDITORWIDGETS_API virtual FReply OnClicked(const FPointerEvent& InMouseEvent);
	ANIMATIONEDITORWIDGETS_API virtual void OnMouseEnter();
	ANIMATIONEDITORWIDGETS_API virtual void OnMouseLeave();
	ANIMATIONEDITORWIDGETS_API virtual void OnDragOver();
	ANIMATIONEDITORWIDGETS_API virtual void OnDragLeave();;

	ANIMATIONEDITORWIDGETS_API virtual FString GetDragDropDecoratorLabel() const;

	template<typename TagType = FSchematicGraphTag>
	TagType* AddTag(bool bNotify = true)
	{
		const TSharedPtr<FSchematicGraphTag> NewTag = MakeShareable(new TagType);
		NewTag->Node = this;
		Tags.Add(NewTag);
		TagByGuid.Add(NewTag->GetGuid(), NewTag);

		if (bNotify)
		{
			NotifyTagAdded(NewTag);
		}
		return static_cast<TagType*>(NewTag.Get());
	}

	template<typename TagType = FSchematicGraphTag>
	const TagType* FindTag(const FGuid& InTagGuid = FGuid()) const
	{
		if(const TSharedPtr<FSchematicGraphTag>* ExistingTag = TagByGuid.Find(InTagGuid))
		{
			return Cast<TagType>(ExistingTag->Get());
		};
		if(!InTagGuid.IsValid())
		{
			for(const TSharedPtr<FSchematicGraphTag>& Tag : Tags)
			{
				if(Tag->IsA<TagType>())
				{
					return Cast<TagType>(Tag.Get());
				}
			}
		}
		return nullptr;
	}

	template<typename TagType = FSchematicGraphTag>
	TagType* FindTag(const FGuid& InTagGuid = FGuid())
	{
		const FSchematicGraphNode* ConstThis = this;
		return const_cast<TagType*>(ConstThis->FindTag<TagType>(InTagGuid));
	}

	template<typename TagType = FSchematicGraphTag>
	const TagType* FindTagChecked(const FGuid& InTagGuid) const
	{
		if(const TSharedPtr<FSchematicGraphTag>* ExistingTag = TagByGuid.Find(InTagGuid))
		{
			check(ExistingTag->Get()->IsA(TagType::Type));
			return static_cast<TagType*>(ExistingTag->Get());
		};
		return nullptr;
	}

	template<typename TagType = FSchematicGraphTag>
	TagType* FindTagChecked(const FGuid& InTagGuid)
	{
		const FSchematicGraphNode* ConstThis = this;
		return const_cast<TagType*>(ConstThis->FindTagChecked<TagType>(InTagGuid));
	}

	ANIMATIONEDITORWIDGETS_API virtual bool RemoveTag(const FGuid& InTagGuid);

	const TArray<TSharedPtr<FSchematicGraphTag>>& GetTags() const { return Tags; }

protected:

	ANIMATIONEDITORWIDGETS_API void NotifyTagAdded(const TSharedPtr<FSchematicGraphTag>& Tag);

	FSchematicGraphModel* Model = nullptr;
	FGuid Guid = FGuid::NewGuid();
	FText Label = FText();
	FGuid ParentNodeGuid = FGuid();
	TArray<FGuid> ChildNodeGuids; 
	bool bIsSelected = false;
	FVector2d Position = FVector2d::ZeroVector;
	FVector2d PositionOffset = FVector2d::ZeroVector;
	float ScaleOffset = 1.f;
	TArray<FLinearColor> Colors;
	TArray<const FSlateBrush*> Brushes;
	FText ToolTip = FText();
	ESchematicGraphVisibility::Type Visibility = ESchematicGraphVisibility::Visible;
	bool bDragSupported = false;

	static inline constexpr float ScaledUp = 1.25;
	static inline constexpr float ScaledDown = 0.75;

	TArray<TSharedPtr<FSchematicGraphTag>> Tags;
	TMap<FGuid, TSharedPtr<FSchematicGraphTag>> TagByGuid;

	friend class FSchematicGraphModel;
};

class FSchematicGraphGroupNode : public FSchematicGraphNode
{
public:

	SCHEMATICGRAPHNODE_BODY(FSchematicGraphGroupNode, FSchematicGraphNode)

	ANIMATIONEDITORWIDGETS_API FSchematicGraphGroupNode();
	virtual ~FSchematicGraphGroupNode() override = default;

	// FSchematicGraphNode interface
	ANIMATIONEDITORWIDGETS_API virtual TOptional<ESchematicGraphVisibility::Type> GetVisibilityForChildNode(const FSchematicGraphNode* InChildNode) const override;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<FVector2d> GetPositionForChildNode(const FSchematicGraphNode* InChildNode) const override;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<float> GetScaleForChildNode(const FSchematicGraphNode* InChildNode) const override;
	ANIMATIONEDITORWIDGETS_API virtual TOptional<bool> GetInteractivityForChildNode(const FSchematicGraphNode* InChildNode) const override;
	ANIMATIONEDITORWIDGETS_API virtual FReply OnClicked(const FPointerEvent& InMouseEvent) override;
	ANIMATIONEDITORWIDGETS_API virtual void OnDragOver() override;
	ANIMATIONEDITORWIDGETS_API virtual void OnDragLeave() override;

	ANIMATIONEDITORWIDGETS_API bool IsExpanded() const;
	ANIMATIONEDITORWIDGETS_API bool IsExpanding() const;
	ANIMATIONEDITORWIDGETS_API bool IsCollapsing() const;
	ANIMATIONEDITORWIDGETS_API float GetExpansionState() const;
	ANIMATIONEDITORWIDGETS_API void SetExpanded(bool InExpanded, bool bAutoCloseParentGroups = false);
	FLinearColor GetExpansionColor() const { return ExpansionColor; }
	void SetExpansionColor(FLinearColor InExpansionColor) { ExpansionColor = InExpansionColor; }
	float GetExpansionRadius() const { return ExpansionRadius; }
	void SetExpansionRadius(float InExpansionRadius) { ExpansionRadius = InExpansionRadius; }
	ANIMATIONEDITORWIDGETS_API float GetDelayDuration(bool bEnter) const;
	void SetDelayDuration(bool bEnter, float InDelayDuration) { (bEnter ? EnterDelayDuration : LeaveDelayDuration) = InDelayDuration; }

protected:

	const TEasingAttributeInterpolator<float>::FSettings& GetAnimationSettings() const { return AnimationSettings; }
	ANIMATIONEDITORWIDGETS_API void SetAnimationSettings(const TEasingAttributeInterpolator<float>::FSettings& InSettings);

	float ExpansionRadius = 60.f;
	FLinearColor ExpansionColor = FLinearColor::White * FLinearColor(1.f, 1.f, 1.f, 0.5f);
	float EnterDelayDuration = 0.25f;
	float LeaveDelayDuration = 0.75f;
	TOptional<bool> LastExpansionState;
	TEasingAttributeInterpolator<float>::FSettings AnimationSettings;
	TSharedPtr<TAnimatedAttribute<float>> ExpansionState;

	friend class SSchematicGraphNode;
};

class FSchematicGraphAutoGroupNode : public FSchematicGraphGroupNode
{
public:

	SCHEMATICGRAPHNODE_BODY(FSchematicGraphAutoGroupNode, FSchematicGraphGroupNode)

	ANIMATIONEDITORWIDGETS_API virtual const FText& GetLabel() const override;

protected:

	mutable FText AutoGroupLabel;
};


#endif
