// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Brushes/SlateRoundedBoxBrush.h"

#define UE_API ANIMATIONEDITORWIDGETS_API

class FSchematicGraphStyle final
	: public FSlateStyleSet
{

public:
	FSchematicGraphStyle()
		: FSlateStyleSet("SchematicGraphStyle")
	{
		const FVector2D Icon10x10(10.0f, 10.0f);
		const FVector2D Icon14x14(14.0f, 14.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon32x32(32.0f, 32.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon128x128(128.0f, 128.0f);
		const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
		SetContentRoot(EngineEditorSlateDir);
		SetCoreContentRoot(EngineEditorSlateDir);

		Set("Schematic.Background", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Background", Icon128x128));
		Set("Schematic.Group", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Group", Icon128x128));
		Set("Schematic.Outline.Single", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Outline_Single", Icon128x128));
		Set("Schematic.Outline.Double", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Outline_Double", Icon128x128));
		Set("Schematic.Dot.Small", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Dot_Small", Icon128x128));
		Set("Schematic.Dot.Medium", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Dot_Medium", Icon128x128));
		Set("Schematic.Dot.Large", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Dot_Large", Icon128x128));
		Set("Schematic.Dot.Group", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Dot_Group", Icon128x128));
		Set("Schematic.Tag.Background", new IMAGE_BRUSH_SVG("SchematicGraph/Schematic_Tag_Background", Icon128x128));

		Set("Schematic.Label.Background", new FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Transparent, 0.0f));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static UE_API FSchematicGraphStyle& Get();
	
	~FSchematicGraphStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};

#undef UE_API
