// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "SchematicGraphDefines.h"

#define SCHEMATICGRAPHTAG_BODY(ClassName, SuperClass) \
SCHEMATICGRAPHELEMENT_BODY(ClassName, SuperClass, FSchematicGraphTag)

class FSchematicGraphTag : public TSharedFromThis<FSchematicGraphTag>
{
public:
	
	SCHEMATICGRAPHELEMENT_BODY_BASE(FSchematicGraphTag)

	ANIMATIONEDITORWIDGETS_API FSchematicGraphTag();

	ANIMATIONEDITORWIDGETS_API const FSchematicGraphNode* GetNode() const;
	const FGuid& GetGuid() const { return Guid; }
	virtual FLinearColor GetBackgroundColor() const { return BackgroundColor; }
	virtual void SetBackgroundColor(const FLinearColor& InBackgroundColor) { BackgroundColor = InBackgroundColor; }
	virtual FLinearColor GetForegroundColor() const { return ForegroundColor; }
	virtual void SetForegroundColor(const FLinearColor& InForegroundColor) { ForegroundColor = InForegroundColor; }
	virtual FLinearColor GetLabelColor() const { return LabelColor; }
	virtual void SetLabelColor(const FLinearColor& InLabelColor) { LabelColor = InLabelColor; }
	virtual const FSlateBrush* GetBackgroundBrush() const { return BackgroundBrush; }
	virtual void SetBackgroundBrush(const FSlateBrush* InBackgroundBrush) { BackgroundBrush = InBackgroundBrush; }
	virtual const FSlateBrush* GetForegroundBrush() const { return ForegroundBrush; }
	virtual void SetForegroundBrush(const FSlateBrush* InForegroundBrush) { ForegroundBrush = InForegroundBrush; }
	virtual FText GetLabel() const { return Label; }
	virtual void SetLabel(const FText& InLabel) { Label = InLabel; }
	virtual FText GetToolTip() const { return ToolTip; }
	virtual void SetToolTip(const FText& InToolTip) { ToolTip = InToolTip; }
	virtual float GetPlacementAngle() const { return PlacementAngle; }
	virtual void SetPlacementAngle(float InPlacementAngle) { PlacementAngle = InPlacementAngle; }
	virtual ESchematicGraphVisibility::Type GetVisibility() const { return Visibility; }
	virtual void SetVisibility(ESchematicGraphVisibility::Type InVisibility) { Visibility = InVisibility; }

protected:

	FSchematicGraphNode* Node = nullptr;
	FGuid Guid = FGuid::NewGuid();
	FLinearColor BackgroundColor = FLinearColor(FColor(0, 112, 224));
	FLinearColor ForegroundColor = FLinearColor::White;
	FLinearColor LabelColor = FLinearColor::White;
	const FSlateBrush* BackgroundBrush = nullptr;
	const FSlateBrush* ForegroundBrush = nullptr;
	FText Label = FText();
	FText ToolTip = FText();
	float PlacementAngle = -45.f;
	ESchematicGraphVisibility::Type Visibility = ESchematicGraphVisibility::Visible;

	friend class FSchematicGraphModel;
	friend class FSchematicGraphNode;
};

class FSchematicGraphGroupTag : public FSchematicGraphTag
{
public:

	SCHEMATICGRAPHTAG_BODY(FSchematicGraphGroupTag, FSchematicGraphTag)

	virtual ~FSchematicGraphGroupTag() override = default;

	ANIMATIONEDITORWIDGETS_API virtual ESchematicGraphVisibility::Type GetVisibility() const override;
	ANIMATIONEDITORWIDGETS_API virtual FText GetLabel() const override;
};

#endif
