// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR

#pragma once

#include "CoreMinimal.h"
#include "SSchematicGraphPanel.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

/** Called back when a viewport is created */
DECLARE_DELEGATE_OneParam(FOnSchematicViewportCreated, const TSharedRef<class SSchematicGraphPanel>&);

/** Arguments used to create a persona viewport tab */
struct FSchematicViewportArgs
{
	FSchematicViewportArgs()
	{}

	/** The model which contains the graph to display */
	TWeakPtr<FSchematicGraphModel> SchematicGraphWeak;
	
	/** Delegate fired when the viewport is created */
	FOnSchematicViewportCreated OnViewportCreated;
};


struct FSchematicViewportTabSummoner : public FWorkflowTabFactory
{
public:
	static ANIMATIONEDITORWIDGETS_API const FName TabID;
	
public:
	ANIMATIONEDITORWIDGETS_API FSchematicViewportTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const FSchematicViewportArgs& InArgs);
	
	ANIMATIONEDITORWIDGETS_API virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const;
	ANIMATIONEDITORWIDGETS_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	TWeakPtr<FSchematicGraphModel> SchematicGraphWeak;
	FOnSchematicViewportCreated OnViewportCreated;
};
#endif
